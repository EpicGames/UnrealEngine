[CoreRedirects]
; ERivermaxMediaOutputPixelFormat was unified into ERivermaxPixelFormat (UE 5.8).
; Old URivermaxMediaOutput assets at version >= AncTimecode(1) have VideoStream.PixelFormat
; stored with the old type tag. Old assets at version < AncTimecode have PixelFormat_DEPRECATED
; stored with the old type tag. Value names changed (e.g. PF_10BIT_RGB -> RGB_10bit).
+EnumRedirects=(OldName="ERivermaxMediaOutputPixelFormat",NewName="ERivermaxPixelFormat",ValueChanges=(("PF_8BIT_YUV422","YUV422_8bit"),("PF_10BIT_YUV422","YUV422_10bit"),("PF_8BIT_RGB","RGB_8bit"),("PF_10BIT_RGB","RGB_10bit"),("PF_12BIT_RGB","RGB_12bit"),("PF_FLOAT16_RGB","RGB_16bit_Float")))

; ERivermaxMediaSourcePixelFormat was unified into ERivermaxPixelFormat (UE 5.8).
; Old URivermaxMediaSource assets at version < VideoStreamStruct(1) have the flat PixelFormat
; property stored with the old type tag. Value names are identical; only the class name changed.
+EnumRedirects=(OldName="ERivermaxMediaSourcePixelFormat",NewName="ERivermaxPixelFormat")
