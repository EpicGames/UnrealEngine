{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "NVIDIA Rivermax Media Streaming",
	"Description": "Adding NVIDIA Rivermax capabilities for Media Captures and Media Players",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"Hidden": "false",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64", "Linux" ],
	"Plugins": [
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		},
		{
			"Name": "MediaPlayerEditor",
			"Enabled": true
		},
		{
			"Name": "nDisplayModularFeatures",
			"Enabled": true
		},
		{
			"Name": "RivermaxCore",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "RivermaxMedia",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Linux" ]
		},
		{
			"Name": "RivermaxMediaEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Linux" ]
		},
		{
			"Name": "RivermaxMediaFactory",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Linux" ]
		}
	]
}