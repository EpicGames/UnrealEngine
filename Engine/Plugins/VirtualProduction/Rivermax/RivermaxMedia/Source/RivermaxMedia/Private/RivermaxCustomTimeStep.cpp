// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxCustomTimeStep.h"

#include "IRivermaxBoundaryMonitor.h"
#include "IRivermaxCoreModule.h"
#include "IRivermaxManager.h"
#include "RivermaxMediaLog.h"
#include "RivermaxPTPUtils.h"
#include "Stats/StatsMisc.h"


#define LOCTEXT_NAMESPACE "RivermaxCustomTimeStep"



bool URivermaxCustomTimeStep::Initialize(UEngine* InEngine)
{
	IRivermaxCoreModule* RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
	if (RivermaxModule == nullptr)
	{
		State = ECustomTimeStepSynchronizationState::Error;
		return false;
	}
	
	State = ECustomTimeStepSynchronizationState::Synchronizing;

	if (RivermaxModule->GetRivermaxManager()->IsManagerInitialized() == false)
	{
		RivermaxModule->GetRivermaxManager()->OnPostRivermaxManagerInit().AddUObject(this, &URivermaxCustomTimeStep::OnRivermaxManagerInitialized);
	}
	else
	{
		OnRivermaxManagerInitialized();
	}

	return true;
}

void URivermaxCustomTimeStep::Shutdown(UEngine* InEngine)
{
#if WITH_EDITORONLY_DATA
	InitializedEngine = nullptr;
#endif

	State = ECustomTimeStepSynchronizationState::Closed;

	IRivermaxCoreModule* RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
	if (RivermaxModule)
	{
		DisableMonitoring();
		RivermaxModule->GetRivermaxManager()->OnPostRivermaxManagerInit().RemoveAll(this);
	}
}

bool URivermaxCustomTimeStep::UpdateTimeStep(UEngine* InEngine)
{
	IRivermaxCoreModule* RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
	if (RivermaxModule == nullptr)
	{
		State = ECustomTimeStepSynchronizationState::Error;
	}

	if (State == ECustomTimeStepSynchronizationState::Closed || State == ECustomTimeStepSynchronizationState::Synchronizing)
	{
		return true;
	}

	if (State == ECustomTimeStepSynchronizationState::Error)
	{
		bIsPreviousFrameNumberValid = false;
		return true;
	}

	// Updates logical last time to match logical current time from last tick
	UpdateApplicationLastTime();

	const double TimeBeforeSync = FPlatformTime::Seconds();

	const bool bWaitedForSync = WaitForSync();

	const double TimeAfterSync = FPlatformTime::Seconds();

	if (!bWaitedForSync)
	{
		return true;
	}

	UpdateAppTimes(TimeBeforeSync, TimeAfterSync);

	return false;
}

ECustomTimeStepSynchronizationState URivermaxCustomTimeStep::GetSynchronizationState() const
{
	return State;
}

FFrameRate URivermaxCustomTimeStep::GetFixedFrameRate() const
{
	return FrameRate;
}

uint32 URivermaxCustomTimeStep::GetLastSyncCountDelta() const
{
	return DeltaFrameNumber;
}

bool URivermaxCustomTimeStep::IsLastSyncDataValid() const
{
	return bIsPreviousFrameNumberValid;
}

FFrameRate URivermaxCustomTimeStep::GetSyncRate() const
{
	FFrameRate SyncRate = GetFixedFrameRate();
	//Need to handle SyncRate times 2 for interlace format
	return SyncRate;
}

bool URivermaxCustomTimeStep::WaitForSync()
{
	DeltaFrameNumber = 1;

	const bool bWasWaitValid = WaitForNextFrame();
	if (!bWasWaitValid)
	{
		bIsPreviousFrameNumberValid = false;
		if (!bIgnoreWarningForOneFrame)
		{
			UE_LOGF(LogRivermaxMedia, Error, "Rivermax CustomTimeStep wait for sync timed out.");
		}

		return false;
	}

	IRivermaxCoreModule& RivermaxModule = FModuleManager::GetModuleChecked<IRivermaxCoreModule>("RivermaxCore");
	const uint64 CurrentPTPTimeNanosec = RivermaxModule.GetRivermaxManager()->GetTime();
	const uint64 NewFrameNumber = UE::RivermaxCore::GetFrameNumber(CurrentPTPTimeNanosec, FrameRate);
	const uint32 ExpectedFramesPerWait = GetExpectedSyncCountDelta();

	if (bEnableOverrunDetection
		&& bIsPreviousFrameNumberValid
		&& (NewFrameNumber != (PreviousFrameNumber + ExpectedFramesPerWait)))
	{
		UE_LOGF(LogRivermaxMedia, Warning,
			"The Engine couldn't run fast enough to keep up with the CustomTimeStep Sync. '%lld' frame(s) dropped.",
			NewFrameNumber - PreviousFrameNumber - ExpectedFramesPerWait);
	}

	{
		if (bIsPreviousFrameNumberValid)
		{
			DeltaFrameNumber = NewFrameNumber - PreviousFrameNumber;
		}

		PreviousFrameNumber = NewFrameNumber;

		bIsPreviousFrameNumberValid = true;
	}

	return true;
}

#if WITH_EDITOR
void URivermaxCustomTimeStep::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URivermaxCustomTimeStep, FrameRate))
	{
		// If frame rate is changing live, reset frame number tracking to somewhat reinitialize our state
		bIsPreviousFrameNumberValid = false;

		// Cleanup and restart monitoring to match frame rate
		EnableMonitoring();
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

bool URivermaxCustomTimeStep::WaitForNextFrame()
{
	static const double TimeoutSec = 5.0;

	IRivermaxCoreModule& RivermaxModule = FModuleManager::GetModuleChecked<IRivermaxCoreModule>("RivermaxCore");
	const uint64 CurrentPTPTime = RivermaxModule.GetRivermaxManager()->GetTime();
	const double CurrentPlatformTime = FPlatformTime::Seconds();
	const uint64 DelayNanosec = AlignmentPointDelayMS * 1E6;
	const uint64 TargetTimeNanosec = UE::RivermaxCore::GetNextAlignmentPoint(CurrentPTPTime, FrameRate) + DelayNanosec;

	double TimeLeftNanosec = 0.0;
	if(TargetTimeNanosec >= CurrentPTPTime)
	{
		TimeLeftNanosec = TargetTimeNanosec - CurrentPTPTime;
	}

	const double WaitTimeSec = FMath::Clamp(TimeLeftNanosec / 1E9, 0.0, TimeoutSec);
	const double StartTime = FPlatformTime::Seconds();
	double ActualWaitTime = 0.0;
	{
		FSimpleScopeSecondsCounter ActualWaitTimeCounter(ActualWaitTime);

		static const float SleepThresholdSec = 5.0f * (1.0f / 1000.0f);
		static const float SpinningTimeSec = 2.0f * (1.0f / 1000.0f);
		if (WaitTimeSec > SleepThresholdSec)
		{
			FPlatformProcess::SleepNoStats(WaitTimeSec - SpinningTimeSec);
		}

		while (FPlatformTime::Seconds() < (CurrentPlatformTime + WaitTimeSec))
		{
			FPlatformProcess::SleepNoStats(0.f);
		}
	}

	return ActualWaitTime < TimeoutSec;
}

void URivermaxCustomTimeStep::OnRivermaxManagerInitialized()
{
	if (State == ECustomTimeStepSynchronizationState::Synchronizing)
	{
		IRivermaxCoreModule& RivermaxModule = FModuleManager::GetModuleChecked<IRivermaxCoreModule>("RivermaxCore");

		if (RivermaxModule.GetRivermaxManager()->ValidateLibraryIsLoaded() == false)
		{
			UE_LOGF(LogRivermaxMedia, Warning, "Can't initialize Rivermax custom timestep. Library isn't available.");
			
			State = ECustomTimeStepSynchronizationState::Error;
			return;
		}

		if (RivermaxModule.GetRivermaxManager()->GetDevices().Num() <= 0)
		{
			UE_LOGF(LogRivermaxMedia, Warning, "Can't initialize Rivermax custom timestep. No devices were found.")
			State = ECustomTimeStepSynchronizationState::Error;
			return;
		}

		EnableMonitoring();

		State = ECustomTimeStepSynchronizationState::Synchronized;
	}
}

void URivermaxCustomTimeStep::EnableMonitoring()
{
	if (IRivermaxCoreModule* RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore"))
	{
		if (FrameRate != CachedFrameRate)
		{
			DisableMonitoring();

			if (FrameRate.IsValid())
			{
				BoundaryMonitorListener = RivermaxModule->GetRivermaxBoundaryMonitor().StartMonitoring(FrameRate);
				CachedFrameRate = FrameRate;
			}
		}
	}
}

void URivermaxCustomTimeStep::DisableMonitoring()
{
	if (IRivermaxCoreModule* RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore"))
	{
		if (BoundaryMonitorListener.IsValid())
		{
			RivermaxModule->GetRivermaxBoundaryMonitor().StopMonitoring(BoundaryMonitorListener, CachedFrameRate);
			BoundaryMonitorListener.Invalidate();
		}
	}
}

#undef LOCTEXT_NAMESPACE
