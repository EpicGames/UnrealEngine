// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxMediaCapture.h"

#include "Async/Async.h"
#include "Async/Fundamental/Task.h"
#include "IRivermaxCoreModule.h"
#include "IRivermaxManager.h"
#include "IRivermaxOutputStream.h"
#include "RenderGraphUtils.h"
#include "RivermaxMediaLog.h"
#include "RivermaxMediaOutput.h"
#include "RivermaxMediaUtils.h"
#include "RivermaxShaders.h"
#include "RivermaxTracingUtils.h"
#include "Slate/SceneViewport.h"
#include "Widgets/SViewport.h"

#if WITH_EDITOR
#include "AnalyticsEventAttribute.h"
#include "EngineAnalytics.h"
#endif


DECLARE_GPU_STAT(Rmax_Capture);
DECLARE_GPU_STAT(Rmax_FrameReservation);


/* namespace RivermaxMediaCaptureDevice
*****************************************************************************/

namespace UE::RivermaxMediaCaptureUtil
{
	void GetOutputEncodingInfo(ERivermaxPixelFormat InPixelFormat, const FIntPoint& InSize, uint32& OutBytesPerElement, uint32& OutElementsPerFrame)
	{
		using namespace UE::RivermaxCore;

		const ESamplingType SamplingType = UE::RivermaxMediaUtils::Private::PixelFormatToRivermaxSamplingType(InPixelFormat);
		const FVideoFormatInfo Info = FStandardVideoFormat::GetVideoFormatInfo(SamplingType);

		// Compute horizontal byte count (stride) of aligned resolution
		const uint32 PixelAlignment = Info.PixelGroupCoverage;
		const FIntPoint AlignedResolution = UE::RivermaxMediaUtils::Private::GetAlignedResolution(Info, InSize);
		const uint32 PixelCount = AlignedResolution.X * AlignedResolution.Y;
		const uint32 PixelGroupCount = PixelCount / PixelAlignment;
		const uint32 FrameByteCount = PixelGroupCount * Info.PixelGroupSize;

		switch (InPixelFormat)
		{
		case ERivermaxPixelFormat::YUV422_8bit:
		{
			OutBytesPerElement = sizeof(UE::RivermaxShaders::FRGBToYUV8Bit422CS::FYUV8Bit422Buffer);
			break;
		}
		case ERivermaxPixelFormat::YUV422_10bit:
		{
			OutBytesPerElement = sizeof(UE::RivermaxShaders::FRGBToYUV10Bit422LittleEndianCS::FYUV10Bit422LEBuffer);
			break;
		}
		case ERivermaxPixelFormat::RGB_8bit:
		{
			OutBytesPerElement = sizeof(UE::RivermaxShaders::FRGBToRGB8BitCS::FRGB8BitBuffer);
			break;
		}
		case ERivermaxPixelFormat::RGB_10bit:
		{
			OutBytesPerElement = sizeof(UE::RivermaxShaders::FRGBToRGB10BitCS::FRGB10BitBuffer);
			break;
		}
		case ERivermaxPixelFormat::RGB_12bit:
		{
			OutBytesPerElement = sizeof(UE::RivermaxShaders::FRGBToRGB12BitCS::FRGB12BitBuffer);
			break;
		}
		case ERivermaxPixelFormat::RGB_16bit_Float:
		{
			OutBytesPerElement = sizeof(UE::RivermaxShaders::FRGBToRGB16fCS::FRGB16fBuffer);
			break;
		}
		}

		// Shader encoding might not align with pixel group size so we need to have enough elements to represent the last pixel group
		OutElementsPerFrame = FMath::CeilToInt32((float)FrameByteCount / OutBytesPerElement);
	}
}


///* URivermaxMediaCapture implementation
//*****************************************************************************/

UE::RivermaxCore::FRivermaxOutputOptions URivermaxMediaCapture::GetOutputOptions() const
{
	return Options;
}

void URivermaxMediaCapture::GetLastPresentedFrameInformation(UE::RivermaxCore::FPresentedFrameInfo& OutFrameInfo) const
{
	using namespace UE::RivermaxCore;
	if (Streams[ERivermaxStreamType::ST2110_20].IsValid())
	{
		Streams[ERivermaxStreamType::ST2110_20]->GetLastPresentedFrame(OutFrameInfo);
	}
}

void URivermaxMediaCapture::WaitForGPU(FRHITexture* InRHITexture)
{
	if (bIsActive && ShouldCaptureRHIResource())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(URivermaxMediaCapture::GPUDirect_WaitForGPU);
		while (!ShaderCompletedRenderingFence->Poll())
		{
			FPlatformProcess::YieldThread();
		}
		ShaderCompletedRenderingFence->Clear();
		GPUWaitCompleteEvent->Trigger();
	}
}

bool URivermaxMediaCapture::ValidateMediaOutput() const
{
	URivermaxMediaOutput* RivermaxMediaOutput = Cast<URivermaxMediaOutput>(MediaOutput);
	if (!RivermaxMediaOutput)
	{
		UE_LOGF(LogRivermaxMedia, Error, "Can not start the capture. MediaOutput's class is not supported.");
		return false;
	}

	return true;
}

bool URivermaxMediaCapture::InitializeCapture()
{
	URivermaxMediaOutput* RivermaxOutput = CastChecked<URivermaxMediaOutput>(MediaOutput);
	UE::RivermaxCore::FRivermaxOutputOptions MediaOutputOptions = RivermaxOutput->GenerateStreamOptions();
	bool bResult = Initialize(MediaOutputOptions);
	return bResult;
}

bool URivermaxMediaCapture::UpdateSceneViewportImpl(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	return true;
}

bool URivermaxMediaCapture::UpdateRenderTargetImpl(UTextureRenderTarget2D* InRenderTarget)
{
	return true;
}

void URivermaxMediaCapture::StopCaptureImpl(bool bAllowPendingFrameToBeProcess)
{
	bIsActive = false;

	for (TUniquePtr<UE::RivermaxCore::IRivermaxOutputStream>& Stream : Streams)
	{
		if (Stream.IsValid())
		{
			Stream->Uninitialize();
			Stream.Reset();
		}
	}
}

bool URivermaxMediaCapture::ShouldCaptureRHIResource() const
{
	using namespace UE::RivermaxCore;
	if (Streams[ERivermaxStreamType::ST2110_20].IsValid())
	{
		return Streams[ERivermaxStreamType::ST2110_20]->IsGPUDirectSupported();
	}

	return false;
}

bool URivermaxMediaCapture::HasFinishedProcessing() const
{
	return Super::HasFinishedProcessing();
}

bool URivermaxMediaCapture::Initialize(const UE::RivermaxCore::FRivermaxOutputOptions& InMediaOutputOptions)
{
	using namespace UE::RivermaxCore;
	Options = InMediaOutputOptions;
	bIsActive = false;

	IRivermaxCoreModule* Module = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
	if (!Module)
	{
		return false;
	}

	TArray<char> SDP;
	if (!ConfigureCapture() || !UE::RivermaxMediaUtils::Private::StreamOptionsToSDPDescription(Options, SDP))
	{
		return false;
	}

	bool bStreamsInitialized = false;
	for (uint8 StreamId = 0; StreamId < ERivermaxStreamType::MAX; ++StreamId)
	{
		const ERivermaxStreamType StreamType = static_cast<ERivermaxStreamType>(StreamId);
		if (Options.GetStreamOptions(StreamType).IsValid())
		{
			// If this fails it means that we don't support this stream type yet.
			Streams[StreamId] = Module->CreateOutputStream(StreamType, SDP);

			if (Streams[StreamId].IsValid())
			{
				bStreamsInitialized = Streams[StreamId]->Initialize(Options, *this);
				if (!bStreamsInitialized)
				{
					UE_LOGF(LogRivermaxMedia, Error, "Couldn't initialize stream ERivermaxStreamType: %d", StreamType);
					break;
				}
			}
			else
			{
				bStreamsInitialized = false;
				UE_LOGF(LogRivermaxMedia, Error, "Unable to create stream ERivermaxStreamType: %d", StreamType);
				break;
			}
		}
	}

	bIsActive = bStreamsInitialized;

	// All streams should be initialized. If at least one fails - stop.
	if (bStreamsInitialized == false)
	{
		StopCaptureImpl(false /*bAllowPendingFrameToBeProcess*/);
		return false;
	}
	

	ShaderCompletedRenderingFence = RHICreateGPUFence(TEXT("RmaxRenderingCompleteFence"));
	GPUWaitCompleteEvent = MakeShareable(FPlatformProcess::GetSynchEventFromPool(true /*bIsManualReset*/), [](FEvent* EventToDelete)
		{
			FPlatformProcess::ReturnSynchEventToPool(EventToDelete);
		});

	// This event starts with a triggered state because Render thread expects this to be triggered to issue render commands.
	// Since nothing has been completed yet, this needs to be triggered.
	GPUWaitCompleteEvent->Trigger();

	return bIsActive;
}

bool URivermaxMediaCapture::ConfigureCapture() const
{
	using namespace UE::RivermaxCore;

	// Resolve interface address
	IRivermaxCoreModule* Module = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
	if (Module == nullptr)
	{
		return false;
	}

	if (!Module->GetRivermaxManager()->ValidateLibraryIsLoaded())
	{
		return false;
	}
	TSharedPtr<FRivermaxVideoOutputOptions> VideoOptions
		= StaticCastSharedPtr<FRivermaxVideoOutputOptions>(Options.StreamOptions[ERivermaxStreamType::ST2110_20]);

	// Video configuration
	if (VideoOptions.IsValid())
	{
		const bool bFoundDevice = Module->GetRivermaxManager()->GetMatchingDevice(VideoOptions->InterfaceAddress, VideoOptions->InterfaceAddress);
		if (bFoundDevice == false)
		{
			UE_LOGF(LogRivermaxMedia, Error, "Could not find a matching interface for IP '%ls'", *VideoOptions->InterfaceAddress);
			return false;
		}
		
		/** Override the size. */
		VideoOptions->Resolution = GetDesiredSize();

		if (VideoOptions->Resolution.X <= 0 || VideoOptions->Resolution.Y <= 0)
		{
			UE_LOGF(LogRivermaxMedia, Warning, "Can't start capture. Invalid resolution requested: %dx%d", VideoOptions->Resolution.X, VideoOptions->Resolution.Y);
			return false;
		}

		const FVideoFormatInfo Info = FStandardVideoFormat::GetVideoFormatInfo(VideoOptions->PixelFormat);
		VideoOptions->AlignedResolution = UE::RivermaxMediaUtils::Private::GetAlignedResolution(Info, VideoOptions->Resolution);
	}

	TSharedPtr<FRivermaxAncOutputOptions> AncOptions
		= StaticCastSharedPtr<FRivermaxAncOutputOptions>(Options.StreamOptions[ERivermaxStreamType::ST2110_40_TC]);
	if (AncOptions.IsValid())
	{
		const bool bFoundDevice = Module->GetRivermaxManager()->GetMatchingDevice(AncOptions->InterfaceAddress, AncOptions->InterfaceAddress);
		if (bFoundDevice == false)
		{
			UE_LOGF(LogRivermaxMedia, Error, "Could not find a matching interface for IP '%ls'", *AncOptions->InterfaceAddress);
			return false;
		}
	}
	return true;
}

void URivermaxMediaCapture::AddFrameReservationPass(FRDGBuilder& GraphBuilder)
{
	RHI_BREADCRUMB_EVENT_STAT(GraphBuilder.RHICmdList, Rmax_FrameReservation, "Rmax_FrameReservation");
	
	// Since we are going to enqueue a lambda that can potentially sleep in the RHI thread if the pixels haven't arrived,
	// we dispatch the existing commands before any potential sleep.
	GraphBuilder.RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);

	// Scene rendering will already be enqueued but capture conversion pass will not
	// Revisit to push slot reservation till last minute
	URivermaxMediaCapture* Capturer = this;
	GraphBuilder.RHICmdList.EnqueueLambda([Capturer, FrameCounter = GFrameCounterRenderThread](FRHICommandList& RHICmdList)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RmaxFrameReservation);
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*UE::RivermaxCore::FRivermaxTracingUtils::RmaxOutMediaCapturePipeTraceEvents[FrameCounter % 10]);

		using namespace UE::RivermaxCore;
		if (Capturer->bIsActive)
		{
			using namespace UE::RivermaxCore;
			for (uint8 StreamId = 0; StreamId < ERivermaxStreamType::MAX; ++StreamId)
			{
				if (Capturer->Streams[StreamId].IsValid())
				{
					Capturer->Streams[StreamId]->ReserveFrame(FrameCounter);
				}
			}
		}
	});
}

void URivermaxMediaCapture::OnFrameCapturedInternal_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow)
{
	using namespace UE::RivermaxCore;
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FRivermaxTracingUtils::RmaxOutMediaCapturePipeTraceEvents[InBaseData.SourceFrameNumberRenderThread % 10]);

	TSharedPtr<FRivermaxOutputInfoVideo> NewFrameVideo = MakeShared<FRivermaxOutputInfoVideo>();
	NewFrameVideo->Height = Height;
	NewFrameVideo->Width = Width;
	NewFrameVideo->Stride = BytesPerRow;
	NewFrameVideo->CPUBuffer = InBuffer;
	NewFrameVideo->FrameIdentifier = InBaseData.SourceFrameNumberRenderThread;

	// Video stream is the requirement for Media capture, therefore it is always valid.
	if (!Streams[ERivermaxStreamType::ST2110_20].IsValid() || Streams[ERivermaxStreamType::ST2110_20]->PushFrame(NewFrameVideo) == false)
	{
		UE_LOGF(LogRivermaxMedia, Verbose, "Failed to push captured frame");
	}

	if (Streams[ERivermaxStreamType::ST2110_40_TC].IsValid())
	{
		TSharedPtr<FRivermaxAncOutputOptions> AncOptions
			= StaticCastSharedPtr<FRivermaxAncOutputOptions>(Options.StreamOptions[ERivermaxStreamType::ST2110_40_TC]);
		TSharedPtr<FRivermaxOutputInfoAncTimecode> NewFrameAnc = MakeShared<FRivermaxOutputInfoAncTimecode>();
		NewFrameAnc->FrameIdentifier = InBaseData.SourceFrameNumberRenderThread;
		NewFrameAnc->Timecode = InBaseData.SourceFrameTimecode;
		NewFrameAnc->FrameRate = AncOptions->FrameRate;

		Streams[ERivermaxStreamType::ST2110_40_TC]->PushFrame(NewFrameAnc);
	}
}

void URivermaxMediaCapture::OnRHIResourceCapturedInternal_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FBufferRHIRef InBuffer)
{
	using namespace UE::RivermaxCore;
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FRivermaxTracingUtils::RmaxOutMediaCapturePipeTraceEvents[InBaseData.SourceFrameNumberRenderThread % 10]);
	TSharedPtr<FRivermaxOutputInfoVideo> NewFrameVideo = MakeShared<FRivermaxOutputInfoVideo>();
	NewFrameVideo->FrameIdentifier = InBaseData.SourceFrameNumberRenderThread;
	NewFrameVideo->GPUBuffer = InBuffer;

	// Video stream is the requirement for Media capture, therefore it is always valid.
	if (!Streams[ERivermaxStreamType::ST2110_20].IsValid() || Streams[ERivermaxStreamType::ST2110_20]->PushFrame(NewFrameVideo) == false)
	{
		UE_LOGF(LogRivermaxMedia, Verbose, "Failed to push captured frame");
	}

	if (Streams[ERivermaxStreamType::ST2110_40_TC].IsValid())
	{
		TSharedPtr<FRivermaxAncOutputOptions> AncOptions
			= StaticCastSharedPtr<FRivermaxAncOutputOptions>(Options.StreamOptions[ERivermaxStreamType::ST2110_40_TC]);
		TSharedPtr<FRivermaxOutputInfoAncTimecode> NewFrameAnc = MakeShared<FRivermaxOutputInfoAncTimecode>();
		NewFrameAnc->FrameIdentifier = InBaseData.SourceFrameNumberRenderThread;
		NewFrameAnc->Timecode = InBaseData.SourceFrameTimecode;
		NewFrameAnc->FrameRate = AncOptions->FrameRate;

		Streams[ERivermaxStreamType::ST2110_40_TC]->PushFrame(NewFrameAnc);
	}
}

void URivermaxMediaCapture::OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(URivermaxMediaCapture::OnFrameCaptured_RenderingThread);
	OnFrameCapturedInternal_AnyThread(InBaseData, InUserData, InBuffer, Width, Height, BytesPerRow);
}

void URivermaxMediaCapture::OnRHIResourceCaptured_RenderingThread(FRHICommandListImmediate& /*RHICmdList*/, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FBufferRHIRef InBuffer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(URivermaxMediaCapture::OnRHIResourceCaptured_RenderingThread);
	OnRHIResourceCapturedInternal_AnyThread(InBaseData, InUserData, InBuffer);
}

void URivermaxMediaCapture::OnRHIResourceCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FBufferRHIRef InBuffer)
{
	using namespace UE::RivermaxCore;

	TRACE_CPUPROFILER_EVENT_SCOPE(URivermaxMediaCapture::OnRHIResourceCaptured_AnyThread);
	OnRHIResourceCapturedInternal_AnyThread(InBaseData, InUserData, InBuffer);
}

void URivermaxMediaCapture::OnFrameCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, const FMediaCaptureResourceData& InResourceData)
{
	using namespace UE::RivermaxCore;

	TRACE_CPUPROFILER_EVENT_SCOPE(URivermaxMediaCapture::OnFrameCaptured_AnyThread);
	OnFrameCapturedInternal_AnyThread(InBaseData, InUserData, InResourceData.Buffer, InResourceData.Width, InResourceData.Height, InResourceData.BytesPerRow);
}

FIntPoint URivermaxMediaCapture::GetCustomOutputSize(const FIntPoint& InSize) const
{
	URivermaxMediaOutput* RivermaxOutput = CastChecked<URivermaxMediaOutput>(MediaOutput);

	uint32 BytesPerElement = 0;
	uint32 ElementsPerFrame = 0;
	UE::RivermaxMediaCaptureUtil::GetOutputEncodingInfo(RivermaxOutput->VideoStream.PixelFormat, InSize, BytesPerElement, ElementsPerFrame);
	return FIntPoint(ElementsPerFrame, 1);
}

EMediaCaptureResourceType URivermaxMediaCapture::GetCustomOutputResourceType() const
{
	URivermaxMediaOutput* RivermaxOutput = CastChecked<URivermaxMediaOutput>(MediaOutput);

	switch (RivermaxOutput->VideoStream.PixelFormat)
	{
	case ERivermaxPixelFormat::YUV422_8bit:
	case ERivermaxPixelFormat::YUV422_10bit:
	case ERivermaxPixelFormat::RGB_8bit:
	case ERivermaxPixelFormat::RGB_10bit:
	case ERivermaxPixelFormat::RGB_12bit:
	case ERivermaxPixelFormat::RGB_16bit_Float:
	{
		return EMediaCaptureResourceType::Buffer; //we use compute shader for all since output format doesn't match texture formats
	}
	default:
		return EMediaCaptureResourceType::Texture;
	}
}

FRDGBufferDesc URivermaxMediaCapture::GetCustomBufferDescription(const FIntPoint& InDesiredSize) const
{
	URivermaxMediaOutput* RivermaxOutput = CastChecked<URivermaxMediaOutput>(MediaOutput);

	uint32 BytesPerElement = 0;
	uint32 ElementsPerFrame = 0;
	UE::RivermaxMediaCaptureUtil::GetOutputEncodingInfo(RivermaxOutput->VideoStream.PixelFormat, InDesiredSize, BytesPerElement, ElementsPerFrame);
	FRDGBufferDesc Desc = FRDGBufferDesc::CreateStructuredDesc(BytesPerElement, ElementsPerFrame);
	
	// Required when GPUDirect using CUDA will be involved
	Desc.Usage |= EBufferUsageFlags::Shared;
	return Desc;
}

void URivermaxMediaCapture::OnCustomCapture_RenderingThread(FRDGBuilder& GraphBuilder, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FRDGTextureRef InSourceTexture, FRDGBufferRef OutputBuffer, const FRHICopyTextureInfo& CopyInfo, FVector2D CropU, FVector2D CropV)
{
	using namespace UE::RivermaxCore;
	RDG_EVENT_SCOPE_STAT(GraphBuilder, Rmax_Capture, "Rmax_Capture");

	TRACE_CPUPROFILER_EVENT_SCOPE(URivermaxMediaCapture::OnCustomCapture_RenderingThread);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FRivermaxTracingUtils::RmaxOutMediaCapturePipeTraceEvents[GFrameCounterRenderThread % 10]);

	using namespace UE::RivermaxShaders;
	URivermaxMediaOutput* RivermaxOutput = CastChecked<URivermaxMediaOutput>(MediaOutput);

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TSharedPtr<FRivermaxVideoOutputOptions> StreamOptions = Options.GetStreamOptions<FRivermaxVideoOutputOptions>(ERivermaxStreamType::ST2110_20);

	if (!ensure(StreamOptions.IsValid()))
	{
		return;
	}

	// Rectangle area to use from source. This is used when source render target is bigger than output resolution
	const FIntRect ViewRect(CopyInfo.GetSourceRect());
	constexpr bool bDoLinearToSRGB = false;
	const FIntPoint AlignedBufferSize = StreamOptions->AlignedResolution;
	const FIntVector GroupCount = UE::RivermaxMediaUtils::Private::GetComputeShaderGroupCount(DesiredOutputSize.X);

	switch (RivermaxOutput->VideoStream.PixelFormat)
	{
	case ERivermaxPixelFormat::YUV422_8bit:
	{
		TShaderMapRef<FRGBToYUV8Bit422CS> ComputeShader(GlobalShaderMap);
		FRGBToYUV8Bit422CS::FParameters* Parameters = ComputeShader->AllocateAndSetParameters(GraphBuilder, InSourceTexture, AlignedBufferSize, ViewRect, DesiredOutputSize, MediaShaders::RgbToYuvRec709Scaled, MediaShaders::YUVOffset8bits, bDoLinearToSRGB, OutputBuffer);

		FComputeShaderUtils::AddPass(
			GraphBuilder
			, RDG_EVENT_NAME("RGBAToYUV8Bit422")
			, ComputeShader
			, Parameters
			, GroupCount);

		break;
	}
	case ERivermaxPixelFormat::YUV422_10bit:
	{
		TShaderMapRef<FRGBToYUV10Bit422LittleEndianCS> ComputeShader(GlobalShaderMap);
		FRGBToYUV10Bit422LittleEndianCS::FParameters* Parameters = ComputeShader->AllocateAndSetParameters(GraphBuilder, InSourceTexture, AlignedBufferSize, ViewRect, DesiredOutputSize, MediaShaders::RgbToYuvRec709Scaled, MediaShaders::YUVOffset10bits, bDoLinearToSRGB, OutputBuffer);

		FComputeShaderUtils::AddPass(
			GraphBuilder
			, RDG_EVENT_NAME("RGBAToYUV10Bit422LE")
			, ComputeShader
			, Parameters
			, GroupCount);
	
		break;
	}
	case ERivermaxPixelFormat::RGB_8bit:
	{
		TShaderMapRef<FRGBToRGB8BitCS> ComputeShader(GlobalShaderMap);
		FRGBToRGB8BitCS::FParameters* Parameters = ComputeShader->AllocateAndSetParameters(GraphBuilder, InSourceTexture, AlignedBufferSize, ViewRect, DesiredOutputSize, OutputBuffer);

		FComputeShaderUtils::AddPass(
			GraphBuilder
			, RDG_EVENT_NAME("RGBAToRGB8Bit")
			, ComputeShader
			, Parameters
			, GroupCount);
		
		break;
	}
	case ERivermaxPixelFormat::RGB_10bit:
	{
		TShaderMapRef<FRGBToRGB10BitCS> ComputeShader(GlobalShaderMap);
		FRGBToRGB10BitCS::FParameters* Parameters = ComputeShader->AllocateAndSetParameters(GraphBuilder, InSourceTexture, AlignedBufferSize, ViewRect, DesiredOutputSize, OutputBuffer);

		FComputeShaderUtils::AddPass(
			GraphBuilder
			, RDG_EVENT_NAME("RGBAToRGB10Bit")
			, ComputeShader
			, Parameters
			, GroupCount);

		break;
	}
	case ERivermaxPixelFormat::RGB_12bit:
	{
		TShaderMapRef<FRGBToRGB12BitCS> ComputeShader(GlobalShaderMap);
		FRGBToRGB12BitCS::FParameters* Parameters = ComputeShader->AllocateAndSetParameters(GraphBuilder, InSourceTexture, AlignedBufferSize, ViewRect, DesiredOutputSize, OutputBuffer);

		FComputeShaderUtils::AddPass(
			GraphBuilder
			, RDG_EVENT_NAME("RGBAToRGB12Bit")
			, ComputeShader
			, Parameters
			, GroupCount);

		break;
	}
	case ERivermaxPixelFormat::RGB_16bit_Float:
	{
		TShaderMapRef<FRGBToRGB16fCS> ComputeShader(GlobalShaderMap);
		FRGBToRGB16fCS::FParameters* Parameters = ComputeShader->AllocateAndSetParameters(GraphBuilder, InSourceTexture, AlignedBufferSize, ViewRect, DesiredOutputSize, OutputBuffer);

		FComputeShaderUtils::AddPass(
			GraphBuilder
			, RDG_EVENT_NAME("RGBAToRGB16f")
			, ComputeShader
			, Parameters
			, GroupCount);

		break;
	}
	}

	//It is only in case of GPU Direct that we need to manually wait for the work to be completed.
	if (ShouldCaptureRHIResource())
	{
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("RivermaxWriteGPUFence"),
			ERDGPassFlags::NeverCull,
			[CompleteFence = ShaderCompletedRenderingFence, InGPUWaitCompleteEvent = GPUWaitCompleteEvent](FRHICommandList& RHICmdList)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RivermaxWriteGPUFence);
				// Render thread could get to this point before Fence is reset. This could cause a deadlock, therefore it 
				// is necessary to wait for Sync thread to complete the copy
				InGPUWaitCompleteEvent->Wait();
				InGPUWaitCompleteEvent->Reset();
				RHICmdList.WriteGPUFence(CompleteFence);
			}
		);
	}

	AddFrameReservationPass(GraphBuilder);
}

#if WITH_EDITOR
FString URivermaxMediaCapture::GetAnalyticsEventPrefix() const
{
	return TEXT("MediaFramework.Rivermax");
}

void URivermaxMediaCapture::GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const
{
	Super::GetAnalyticsEventAttributes(EventName, InOutAttributes);
	
	if (URivermaxMediaOutput* RivermaxOutput = Cast<URivermaxMediaOutput>(MediaOutput))
	{
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("FrameRate"), RivermaxOutput->VideoStream.FrameRate.ToPrettyText().ToString()));
	}
}
#endif

bool URivermaxMediaCapture::IsReadyForFinishDestroy()
{
	return Super::IsReadyForFinishDestroy();
}

void URivermaxMediaCapture::OnInitializationCompleted(bool bHasSucceed)
{
	if (GetState() != EMediaCaptureState::Stopped)
	{
		SetState(bHasSucceed ? EMediaCaptureState::Capturing : EMediaCaptureState::Error);
	}
}

void URivermaxMediaCapture::OnStreamError()
{
	UE_LOGF(LogRivermaxMedia, Error, "Outputstream has caught an error. Stopping capture.");
	if (GetState() != EMediaCaptureState::Stopped)
	{
		SetState(EMediaCaptureState::Error);
	}
}

void URivermaxMediaCapture::OnPreFrameEnqueue()
{
	// Will need to add some logic in that callback chain for the case where margin wasn't enough
	// For now, we act blindly that frames presented are all the same but we need a way to detect 
	// if it's not and correct it.
	TRACE_CPUPROFILER_EVENT_SCOPE(RmaxOutputSynchronization);
	OnOutputSynchronization.ExecuteIfBound();
}
