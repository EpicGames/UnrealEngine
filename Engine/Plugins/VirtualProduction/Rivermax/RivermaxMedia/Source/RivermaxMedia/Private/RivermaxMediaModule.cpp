// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxMediaModule.h"

#include "Modules/ModuleManager.h"
#include "RivermaxMediaPlayer.h"



void FRivermaxMediaModule::StartupModule()
{
}

void FRivermaxMediaModule::ShutdownModule()
{

}

TSharedPtr<IMediaPlayer> FRivermaxMediaModule::CreatePlayer(IMediaEventSink& EventSink)
{
	return MakeShared<UE::RivermaxMedia::FRivermaxMediaPlayer>(EventSink);
}

IMPLEMENT_MODULE(FRivermaxMediaModule, RivermaxMedia);
