// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRivermaxMediaModule.h"

/**
 * 
 */
class FRivermaxMediaModule : public IRivermaxMediaModule
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	virtual TSharedPtr<IMediaPlayer> CreatePlayer(IMediaEventSink& EventSink) override;
};

