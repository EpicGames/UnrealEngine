// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxMediaOutput.h"

#include "Misc/FileHelper.h"
#include "RivermaxMediaCapture.h"
#include "RivermaxMediaLog.h"
#include "RivermaxMediaUtils.h"
#include "Serialization/CustomVersion.h"


enum class ERivermaxMediaOutputVersion : int32
{
	BeforeAncTimecode = 0,
	AncTimecode = 1,
	FrameManagementChange = 2,
	Latest = FrameManagementChange
};

struct FRivermaxMediaOutputCustomVersion
{
	static const FGuid GUID;

	static const int32 LatestVersion = (int32)ERivermaxMediaOutputVersion::Latest;
};

const FGuid FRivermaxMediaOutputCustomVersion::GUID(0xC9C9491C, 0x457EAE01, 0xAA0F1099, 0x94768AC0);

FCustomVersionRegistration GRivermaxMediaOutputCustomVersion(
	FRivermaxMediaOutputCustomVersion::GUID,
	FRivermaxMediaOutputCustomVersion::LatestVersion,
	TEXT("RivermaxMediaOutputVersion")
);

/* URivermaxMediaOutput
*****************************************************************************/

bool URivermaxMediaOutput::Validate(FString& OutFailureReason) const
{
	if (!Super::Validate(OutFailureReason))
	{
		return false;
	}
	
	return true;
}

FIntPoint URivermaxMediaOutput::GetRequestedSize() const
{
	if (VideoStream.bOverrideResolution)
	{
		return VideoStream.Resolution;
	}

	return UMediaOutput::RequestCaptureSourceSize;
}

EPixelFormat URivermaxMediaOutput::GetRequestedPixelFormat() const
{
	// All output types go through buffer conversion
	EPixelFormat Result = EPixelFormat::PF_A2B10G10R10;
	return Result;
}

EMediaCaptureConversionOperation URivermaxMediaOutput::GetConversionOperation(EMediaCaptureSourceType InSourceType) const
{
	EMediaCaptureConversionOperation Result = EMediaCaptureConversionOperation::CUSTOM;
	switch (VideoStream.PixelFormat)
	{
	case ERivermaxPixelFormat::YUV422_8bit:
	case ERivermaxPixelFormat::YUV422_10bit:
	case ERivermaxPixelFormat::RGB_8bit:
	case ERivermaxPixelFormat::RGB_10bit:
	case ERivermaxPixelFormat::RGB_12bit:
	case ERivermaxPixelFormat::RGB_16bit_Float:
	default:
		Result = EMediaCaptureConversionOperation::CUSTOM; //We handle all conversion for rivermax since it's really tied to endianness of 2110
		break;
	}
	return Result;
}

#if WITH_EDITOR
FString URivermaxMediaOutput::GetDescriptionString() const
{
	const FIntPoint Size = GetRequestedSize();
	return FString::Format(TEXT("{0}x{1} {2} fps {3}"), {
		Size.X,
		Size.Y,
		FString::SanitizeFloat(VideoStream.FrameRate.AsDecimal()),
		StaticEnum<ERivermaxPixelFormat>()->GetDisplayNameTextByValue(static_cast<int64>(VideoStream.PixelFormat)).ToString() });
}

void URivermaxMediaOutput::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
#define LOCTEXT_NAMESPACE "RivermaxMediaOutput"
	const FNumberFormattingOptions NumberFormat = FNumberFormattingOptions()
		.DefaultNoGrouping();
	
	const FText ConfigHeader = LOCTEXT("MediaConfigurationHeader", "Media Configuration");

	const FIntPoint Size = GetRequestedSize();
	const FText ResolutionText = FText::Format(LOCTEXT("ResolutionFormat", "{0}x{1}"), FText::AsNumber(Size.X, &NumberFormat), FText::AsNumber(Size.Y, &NumberFormat));
	const FText FramerateText = FText::Format(LOCTEXT("FramerateFormat", "{0} fps"), FText::AsNumber(VideoStream.FrameRate.AsDecimal()));
	const FText PixelFormatText = StaticEnum<ERivermaxPixelFormat>()->GetDisplayNameTextByValue(static_cast<int64>(VideoStream.PixelFormat));
	
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("ResolutionLabel", "Resolution"), ResolutionText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("FramerateLabel", "Framerate"), FramerateText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("PixelFormatLabel", "Pixel Format"), PixelFormatText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("StreamAddressLabel", "Stream Address"), FText::FromString(VideoStream.StreamAddress)));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("PortLabel", "Port"), FText::AsNumber(VideoStream.Port, &NumberFormat)));

	const FText VideoSettingsHeader = LOCTEXT("VideoSettingsHeader", "Video Settings");
	const FText EnabledText = LOCTEXT("Enabled", "Enabled");
	const FText DisabledText = LOCTEXT("Disabled", "Disabled");

	OutInfoElements.Add(FInfoElement(VideoSettingsHeader, LOCTEXT("GPUDirectLabel", "GPUDirect"), VideoStream.bUseGPUDirect ? EnabledText : DisabledText));
	
#undef LOCTEXT_NAMESPACE
}
#endif //WITH_EDITOR

void URivermaxMediaOutput::ExportSDP(const FString& InPath)
{
	TArray<char> SDP;
	UE::RivermaxCore::FRivermaxOutputOptions OutputOptions = GenerateStreamOptions();

	bool bSaved = false;
	if (UE::RivermaxMediaUtils::Private::StreamOptionsToSDPDescription(OutputOptions, SDP))
	{
		FString SdpString(ANSI_TO_TCHAR(SDP.GetData()));

		if (FFileHelper::SaveStringToFile(SdpString, *InPath))
		{
			bSaved = true;
		}
	}

	if (bSaved)
	{
		UE_LOGF(LogTemp, Log, "Saved SDP successfully to: %ls", *InPath);
	}
	else
	{
		UE_LOGF(LogTemp, Error, "Failed to save SDP file: %ls", *InPath);
	}
}

UE::RivermaxCore::FRivermaxOutputOptions URivermaxMediaOutput::GenerateStreamOptions() const
{
	using namespace UE::RivermaxCore;
	using namespace UE::RivermaxMediaUtils::Private;

	UE::RivermaxCore::FRivermaxOutputOptions OutOutputOptions;

	// Video configuration
	{
		TSharedPtr<FRivermaxVideoOutputOptions> VideoOptions
			= StaticCastSharedPtr<FRivermaxVideoOutputOptions>(OutOutputOptions.StreamOptions[ERivermaxStreamType::ST2110_20]);

		// Sanity check. Making sure that Video Options are not intialized prior.
		check(!VideoOptions.IsValid());
		VideoOptions = MakeShared<FRivermaxVideoOutputOptions>();
		OutOutputOptions.StreamOptions[ERivermaxStreamType::ST2110_20] = VideoOptions;

		VideoOptions->InterfaceAddress = VideoStream.InterfaceAddress;
		VideoOptions->StreamAddress = VideoStream.StreamAddress;
		VideoOptions->Port = VideoStream.Port;
		VideoOptions->Resolution = GetRequestedSize();

		VideoOptions->FrameRate = VideoStream.FrameRate;
		OutOutputOptions.NumberOfBuffers = NumberOfTextureBuffers;
		VideoOptions->bUseGPUDirect = VideoStream.bUseGPUDirect;
		OutOutputOptions.AlignmentMode = MediaOutputAlignmentToRivermaxAlignment(AlignmentMode);
		OutOutputOptions.FrameLockingMode = UE::RivermaxMediaUtils::Private::MediaOutputFrameLockingToRivermax(FrameLockingMode);

		// Setup alignment dependent configs
		OutOutputOptions.bDoContinuousOutput = OutOutputOptions.AlignmentMode == ERivermaxAlignmentMode::AlignmentPoint ? bDoContinuousOutput : false;
		OutOutputOptions.bDoFrameCounterTimestamping = OutOutputOptions.AlignmentMode == ERivermaxAlignmentMode::FrameCreation ? bDoFrameCounterTimestamping : false;

		VideoOptions->PixelFormat = UE::RivermaxMediaUtils::Private::PixelFormatToRivermaxSamplingType(VideoStream.PixelFormat);
		const FVideoFormatInfo Info = FStandardVideoFormat::GetVideoFormatInfo(VideoOptions->PixelFormat);
		VideoOptions->AlignedResolution = GetAlignedResolution(Info, VideoOptions->Resolution);
	}

	// DID and SDID map that has ST291 values corresponding to the stream type.
	// Contains DID and SDID values for each ERivermaxAncStreamType
	const TMap<ERivermaxAncStreamType, TPair<uint8, uint8>> MapDIDSDID = 
	{ 
		{ ERivermaxAncStreamType::ST2110_40_TC, {AncTimecodeDID, AncTimecodeSDID}} 
	};

	for (const FRivermaxAncStream & AncStream : AncStreams)
	{
		if (AncStream.StreamType == ERivermaxAncStreamType::None || !MapDIDSDID.Contains(AncStream.StreamType))
		{
			continue;
		}

		UE::RivermaxCore::ERivermaxStreamType StreamType = static_cast<UE::RivermaxCore::ERivermaxStreamType>(AncStream.StreamType);
		TSharedPtr<FRivermaxAncOutputOptions> AncOptions
			= StaticCastSharedPtr<FRivermaxAncOutputOptions>(OutOutputOptions.StreamOptions[StreamType]);

		// Sanity check. Making sure that Anc Options are not intialized prior.
		check(!AncOptions.IsValid());

		TPair<uint8, uint8> DIDSDID = MapDIDSDID[AncStream.StreamType];
		uint8 DID = DIDSDID.Key;
		uint8 SDID = DIDSDID.Value;
		AncOptions = MakeShared<FRivermaxAncOutputOptions>(DID, SDID);
		OutOutputOptions.StreamOptions[StreamType] = AncOptions;

		AncOptions->InterfaceAddress = AncStream.InterfaceAddress;
		AncOptions->StreamAddress = AncStream.StreamAddress;
		AncOptions->Port = AncStream.Port;
		AncOptions->FrameRate = VideoStream.FrameRate;
	}

	return MoveTemp(OutOutputOptions);
}

UMediaCapture* URivermaxMediaOutput::CreateMediaCaptureImpl()
{
	UMediaCapture* Result = NewObject<URivermaxMediaCapture>();
	if (Result)
	{
		Result->SetMediaOutput(this);
	}
	return Result;
}

void URivermaxMediaOutput::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FRivermaxMediaOutputCustomVersion::GUID);
}

void URivermaxMediaOutput::PostLoad()
{
	Super::PostLoad();

	const int32 Version = GetLinkerCustomVersion(FRivermaxMediaOutputCustomVersion::GUID);
	if (Version < (int32)ERivermaxMediaOutputVersion::AncTimecode)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		VideoStream.bOverrideResolution = bOverrideResolution_DEPRECATED;
		VideoStream.Resolution = Resolution_DEPRECATED;
		VideoStream.FrameRate = FrameRate_DEPRECATED; // ANC derives rate from video; keep only here
		VideoStream.PixelFormat = PixelFormat_DEPRECATED;
		VideoStream.InterfaceAddress = InterfaceAddress_DEPRECATED;
		VideoStream.StreamAddress = StreamAddress_DEPRECATED;
		VideoStream.Port = Port_DEPRECATED;
		VideoStream.bUseGPUDirect = bUseGPUDirect_DEPRECATED;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	if (Version < (int32)ERivermaxMediaOutputVersion::FrameManagementChange)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		NumberOfTextureBuffers = PresentationQueueSize_DEPRECATED;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
}


#if WITH_EDITOR
bool URivermaxMediaOutput::CanEditChange(const FProperty* InProperty) const
{
	return Super::CanEditChange(InProperty);
}

void URivermaxMediaOutput::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif //WITH_EDITOR

