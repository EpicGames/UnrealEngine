// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxMediaPlayer.h"

#include "Engine/Engine.h"
#include "IMediaEventSink.h"
#include "IRivermaxCoreModule.h"
#include "IRivermaxManager.h"
#include "MediaIOCoreEncodeTime.h"
#include "MediaIOCoreFileWriter.h"
#include "MediaIOCoreSamples.h"
#include "Misc/ScopeLock.h"
#include "RenderCommandFence.h"
#include "RenderGraphUtils.h"
#include "RivermaxMediaLog.h"
#include "RivermaxMediaOutput.h"
#include "RivermaxMediaSourceOptions.h"
#include "RivermaxMediaTextureSample.h"
#include "RivermaxMediaUtils.h"
#include "RivermaxPTPUtils.h"
#include "RivermaxShaders.h"
#include "RivermaxTracingUtils.h"
#include "RivermaxTimecodeProvider.h"
#include "RivermaxTypes.h"
#include "Stats/Stats.h"
#include "Tasks/Task.h"

#if WITH_EDITOR
#include "EngineAnalytics.h"
#endif


#define LOCTEXT_NAMESPACE "FRivermaxMediaPlayer"

// Sample number with frame delay taken into account
#define TO_EXPECTED_SAMPLE_FRAME_NUM(CURRENTFrameNum) GetFrameNumberWithAcountedLatency(FrameDelay, CURRENTFrameNum)

// Sample index with frame delay taken into account
#define TO_EXPECTED_SAMPLE_INDEX(CURRENTFrameNum) ConvertFrameNumToSampleIndex(FrameDelay, CURRENTFrameNum, kMaxNumVideoFrameBuffer)

// Sample index to access sample in framelocked array
#define FRAME_NUM_TO_INDEX(CURRENTFrameNum) (CURRENTFrameNum) % kMaxNumVideoFrameBuffer

// Identifies if the player is in framelocking mode.
#define IS_FRAMELOCKED() (EvaluationType == EMediaIOSampleEvaluationType::Timecode && bFramelock)

DECLARE_GPU_STAT_NAMED(RivermaxMedia_SampleUsageFence, TEXT("RivermaxMedia_SampleUsageFence"));
DECLARE_GPU_STAT_NAMED(Rmax_WaitForPixels, TEXT("Rmax_WaitForPixels"));
DECLARE_GPU_STAT(RivermaxSource_SampleConversion);

namespace UE::RivermaxMedia
{
	/** Concrete IRivermaxAncSample implementation used by the ANC input stream. */
	class FRivermaxAncSample : public IRivermaxAncSample
	{
	public:
		/** RTP timestamp set when the frame is requested; used to match this ANC sample to its video frame. */
		uint32 RTPTimestamp = 0;

		virtual void SetAncData(uint16 InDID, uint16 InSDID, const TArray<uint16>& InUDWs) override
		{
			DID = InDID;
			SDID = InSDID;
			UDWs = InUDWs;
		}
		virtual uint16 GetDID() const override
		{
			return DID;
		}
		virtual uint16 GetSDID() const override
		{
			return SDID;
		}
		virtual const TArray<uint16>& GetUserDataWords() const override
		{
			return UDWs;
		}

	private:
		uint16 DID = 0;
		uint16 SDID = 0;
		TArray<uint16> UDWs;
	};

	static TAutoConsoleVariable<int32> CVarRivermaxForcedFramelockLatency(
		TEXT("Rivermax.Player.Latency"),
		-1,
		TEXT("Override latency in framelock mode. 0 for 0 frame of latency and 1 for 1 frame of latency."),
		ECVF_Default);

	static TAutoConsoleVariable<float> CVarRivermaxDefaultThreadBlockTimeoutSec(
		TEXT("Rivermax.Player.DefaultThreadBlockTimeoutSec"),
		0.5,
		TEXT("Overrides the wait time for the samples to be written to. In seconds. "),
		ECVF_Default);

	uint64 GetFrameNumberWithAcountedLatency(int64 InFrameDelay, uint64 FrameNumber)
	{
		const int64 ForcedLatency = CVarRivermaxForcedFramelockLatency.GetValueOnAnyThread();
		return (FrameNumber - (uint64)FMath::Clamp(InFrameDelay + ForcedLatency, 0, 1));
	}

	uint8 ConvertFrameNumToSampleIndex(int64 InFrameDelay, uint64 FrameNumber, uint8 InMaxNumVideoFrameBuffer)
	{
		return (GetFrameNumberWithAcountedLatency(InFrameDelay, FrameNumber)) % InMaxNumVideoFrameBuffer;
	}

	/** Returns current time. Adjusted to UTC and rolled over at 24 hours. */
	FTimespan GetCurrentPTPTimeOfDay()
	{
		FTimespan CurrentTimespan;

		const int64 NumberOfTicksPerDay = 60 * 60 * 24 * ETimespan::TicksPerSecond;

		IRivermaxCoreModule* RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
		if (RivermaxModule && RivermaxModule->GetRivermaxManager())
		{
			// Converting from nanoseconds to ticks.
			CurrentTimespan = FTimespan(RivermaxModule->GetRivermaxManager()->GetTime() / ETimespan::NanosecondsPerTick);
			UTimecodeProvider* Provider = GEngine->GetTimecodeProvider();
			
			// Convert from TAI PTP Time to UTC
			if (Provider && Provider->GetName().Contains("RivermaxTimecodeProvider"))
			{
				URivermaxTimecodeProvider* RmaxTimecodeProvider = static_cast<URivermaxTimecodeProvider*>(Provider);
				CurrentTimespan -= FTimespan(0, 0, RmaxTimecodeProvider->UTCSecondsOffset);
			}
			else
			{
				UE_CALL_ONCE([&] { UE_LOGF(LogRivermaxMedia, Warning, "Rivermax Timecode provider is required for accurate playback."); });
			}

			// Rollover 24 hours.
			CurrentTimespan = FTimespan(CurrentTimespan.GetTicks() % NumberOfTicksPerDay);
		}

		return CurrentTimespan;
	}
	/* FRivermaxVideoPlayer structors
	 *****************************************************************************/

	FRivermaxMediaPlayer::FRivermaxMediaPlayer(IMediaEventSink& InEventSink)
		: Super(InEventSink)
		, RivermaxThreadNewState(EMediaState::Closed)
		, VideoTextureSamplePool(MakeUnique<FRivermaxMediaTextureSamplePool>())
	{
	}

	FRivermaxMediaPlayer::~FRivermaxMediaPlayer()
	{
		Close();
	}

	/* IMediaPlayer interface
	 *****************************************************************************/

	 /**
	  * @EventName MediaFramework.RivermaxSourceOpened
	  * @Trigger Triggered when an Rivermax media source is opened through a media player.
	  * @Type Client
	  * @Owner MediaIO Team
	  */
	bool FRivermaxMediaPlayer::Open(const FString& Url, const IMediaOptions* Options)
	{
		if (!Super::Open(Url, Options))
		{
			RivermaxThreadNewState = EMediaState::Error;
			return false;
		}

		//Video related options
		{
			DesiredPixelFormat = (ERivermaxPixelFormat)Options->GetMediaOption(RivermaxMediaOption::PixelFormat, (int64)ERivermaxPixelFormat::RGB_8bit);
			const bool bOverrideResolution = Options->GetMediaOption(RivermaxMediaOption::OverrideResolution, false);
			bFollowsStreamResolution = !bOverrideResolution;
		}

		IRivermaxCoreModule* Module = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
		if (Module && ConfigureStream(Options))
		{
			
			TArray<char> SDP;
			// TODO: Similar to Output stream creation, input streams should be created from the supplied SDP.
			//UE::RivermaxMediaUtils::Private::StreamOptionsToSDPDescription(Options, SDP);
			InputStream = Module->CreateInputStream(UE::RivermaxCore::ERivermaxStreamType::ST2110_20, SDP);
		}

		// If we are not following the stream resolution, make it the video track format and then reset to go through a format change once
		if (!bFollowsStreamResolution)
		{
			StreamResolution = StreamOptions.EnforcedResolution;
		}

		VideoTrackFormat.Dim = FIntPoint::ZeroValue;

		CurrentState = EMediaState::Preparing;
		RivermaxThreadNewState = EMediaState::Preparing;

		if (InputStream == nullptr || !InputStream->Initialize(StreamOptions, *this))
		{
			UE_LOGF(LogRivermaxMedia, Warning, "Failed to initialize Rivermax input stream.");
			RivermaxThreadNewState = EMediaState::Error;
			InputStream.Reset();
			return false;
		}

		// Create ANC input streams for each entry configured in the media source.
		if (Module)
		{
			if (const URivermaxMediaSource* RivermaxSource = Cast<URivermaxMediaSource>(Options->ToUObject()))
			{
				for (const FRivermaxAncStream& AncStreamConfig : RivermaxSource->AncStreams)
				{
					if (AncStreamConfig.StreamType == ERivermaxAncStreamType::None)
					{
						continue;
					}

					const UE::RivermaxCore::ERivermaxStreamType AncType = static_cast<UE::RivermaxCore::ERivermaxStreamType>(AncStreamConfig.StreamType);

					TArray<char> AncSDP;
					TUniquePtr<IRivermaxInputStream> NewAncInputStream = Module->CreateInputStream(AncType, AncSDP);
					if (NewAncInputStream)
					{
						FRivermaxInputStreamOptions AncOptions;
						AncOptions.InterfaceAddress = AncStreamConfig.InterfaceAddress;
						AncOptions.StreamAddress    = AncStreamConfig.StreamAddress;
						AncOptions.Port             = static_cast<uint32>(AncStreamConfig.Port);
						AncOptions.FrameRate        = StreamOptions.FrameRate;
						AncOptions.bUseGPUDirect    = false;

						TUniquePtr<FAncStreamListener> NewListener = MakeUnique<FAncStreamListener>(*this);
						if (NewAncInputStream->Initialize(AncOptions, *NewListener))
						{
							AncInputStreams.Add(MoveTemp(NewAncInputStream));
							AncStreamListeners.Add(MoveTemp(NewListener));
						}
						else
						{
							UE_LOG(LogRivermaxMedia, Warning, TEXT("Failed to initialize ANC input stream."));
						}
					}
				}
			}
		}

		// Setup our different supported channels based on source settings
		SetupSampleChannels();

		return true;
	}

	void FRivermaxMediaPlayer::Close()
	{
		RivermaxThreadNewState = EMediaState::Closed;

		WaitForPendingTasks();

		for (TUniquePtr<IRivermaxInputStream>& AncStream : AncInputStreams)
		{
			if (AncStream)
			{
				AncStream->Uninitialize();
			}
		}
		AncInputStreams.Reset();
		AncStreamListeners.Reset();

		if (InputStream)
		{
			InputStream->Uninitialize(); // this may block, until the completion of a callback from IRivermaxChannelCallbackInterface
			InputStream.Reset();
		}

		Samples->FlushSamples();
		VideoTextureSamplePool.Reset();

		Super::Close();
	}

	FGuid FRivermaxMediaPlayer::GetPlayerPluginGUID() const
	{
		static FGuid PlayerPluginGUID(0xF537595A, 0x8E8D452B, 0xB8C05707, 0x6B334234);
		return PlayerPluginGUID;
	}


#if WITH_EDITOR
	const FSlateBrush* FRivermaxMediaPlayer::GetDisplayIcon() const
	{
		//todo for tdm
		return nullptr;
	}
#endif //WITH_EDITOR

	TSharedPtr<IRivermaxVideoSample> FRivermaxMediaPlayer::OnVideoFrameRequested(const FRivermaxInputVideoFrameDescriptor& FrameInfo)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Rmax::FrameRequested %u"), (FrameInfo.FrameNumber)));

		// If video is not playing, no need to provide samples when requested
		if (!IsReadyToPlay())
		{
			return nullptr;
		}

		if (FrameInfo.VideoBufferSize > 0)
		{
			TSharedPtr<FRivermaxMediaTextureSample> Sample;
			if (IS_FRAMELOCKED())
			{
				// If input stream has somehow completely lost a frame (not even a single packet received) and we are 2 frames ahead of rendered frame.
				if (FrameInfo.FrameNumber > LastFrameToAttemptReception + 1)
				{
					//Invalidate previous samples in case one of them is still being rendered. 
					for (uint64 PreviousFrameNumber = LastFrameToAttemptReception; PreviousFrameNumber < FrameInfo.FrameNumber; PreviousFrameNumber++)
					{
						TSharedPtr<FRivermaxMediaTextureSample> PreviousSample = FrameLockedSamples[FRAME_NUM_TO_INDEX(PreviousFrameNumber)];
						if (PreviousSample->IsBeingRendered())
						{
							// This will stop the waiting thread from waiting for the start of the reception. 
							PreviousSample->SetFrameNumber(PreviousFrameNumber);

							// this will stop the waiting thread from copying invalid data and waiting for the copy.
							PreviousSample->SetReceptionState(IRivermaxSample::ESampleState::ReceptionError);

							// this will stop the waiting thread from waiting for the sample reception.
							PreviousSample->GetSampleReceivedEvent()->Trigger();
						}
					}
				}

				Sample = FrameLockedSamples[FRAME_NUM_TO_INDEX(FrameInfo.FrameNumber)];
			}
			else
			{
				Sample = VideoTextureSamplePool->AcquireShared(false /*NoAllocation*/);
			}

			// (More of a sanity check. Shouldn't be in this state.).
			if (!Sample.IsValid())
			{
				UE_LOGF(LogRivermaxMedia, Warning, "Failed to provide a frame for incoming frame %u with timestamp %u", FrameInfo.FrameNumber, FrameInfo.Timestamp);
				return nullptr;
			}

			// The sample hasn't completed its cycle. This is a sanity check to ensure nothing unexpected has happened with the sample.
			// If this happens it is fine to continue since the sample most likely was skipped by the rendering step.
			if (Sample->GetReceptionState() != IRivermaxVideoSample::ESampleState::Idle)
			{
				const TCHAR*& SampleStateString = FRivermaxMediaTextureSample::SampleStateToStringRef(Sample->GetReceptionState());
				UE_LOGF(LogRivermaxMedia, Warning, "The sample hasn't completed it's cycle. The frame number of the incomplete Sample: %llu Frame Number of the sample about to be received: %u, State: %ls", Sample->GetFrameNumber(), FrameInfo.FrameNumber, SampleStateString);
			}

			// With Unreal as the sender the receiver shouldn't be in situations where it receives the same number twice in a row. However it might be different with other devices.
			// in case such situation is encountered it is good to have something logged.
			if (LastFrameToAttemptReception == FrameInfo.FrameNumber) 
			{
				UE_LOGF(LogRivermaxMedia, Warning, "The same frame number has been received twice in a row. Frame Number: %u, Timestamp: %u", FrameInfo.FrameNumber, FrameInfo.Timestamp);
			}

			UE_LOGF(LogRivermaxMedia, Verbose, "Starting to receive frame '%u' with timestamp %u", FrameInfo.FrameNumber, FrameInfo.Timestamp);

			// Fallback timing: record wall-clock reception window for sample selection when no ANC timecode is available.
			Sample->FrameReceptionStart = GetCurrentPTPTimeOfDay();

			Sample->SetReceptionState(IRivermaxSample::ESampleState::ReadyForReception);
			Sample->SetFrameNumber(FrameInfo.FrameNumber);
			Sample->RTPTimestamp = FrameInfo.Timestamp;
			LastFrameToAttemptReception = FrameInfo.FrameNumber;

			return Sample;
		}

		return nullptr;
	}


	void FRivermaxMediaPlayer::OnVideoFrameReceived(TSharedPtr<IRivermaxVideoSample> InReceivedVideoFrame)
	{
		if (!IsReadyToPlay())
		{
			return;
		}

		TSharedPtr<FRivermaxMediaTextureSample> Sample = StaticCastSharedPtr<FRivermaxMediaTextureSample>(InReceivedVideoFrame);
		check(Sample.IsValid());

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Rmax::OnVideoFrameReceived %llu"), (Sample->GetFrameNumber())));

		// Attaching the ANC timecode (and other ANC data in the future) for this video frame. 
		{
			FScopeLock Lock(&PendingAncDataLock);
			if (const FVideoFrameAncData* AncData = PendingAncData.Find(Sample->RTPTimestamp))
			{
				Sample->SetTimecode(AncData->Timecode);
				PendingAncData.Remove(Sample->RTPTimestamp);
			}
		}

		if (!IS_FRAMELOCKED())
		{
			Samples->AddVideo(Sample.ToSharedRef());
		}
		else
		{
			Sample->GetSampleReceivedEvent()->Trigger();
		}

		Sample->SetReceptionState(IRivermaxSample::ESampleState::Received);

		// Fallback timing: record wall-clock reception window for sample selection when no ANC timecode is available.
		Sample->FrameReceptionEnd = GetCurrentPTPTimeOfDay();
		
	}

	void FRivermaxMediaPlayer::TickInput(FTimespan DeltaTime, FTimespan Timecode)
	{
		// update player state
		EMediaState NewState = RivermaxThreadNewState;

		if (NewState != CurrentState)
		{
			CurrentState = NewState;
			if (CurrentState == EMediaState::Playing)
			{
				EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
				EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
				EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);
			}
			else if (NewState == EMediaState::Error)
			{
				EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
				Close();
			}
		}

		if (CurrentState != EMediaState::Playing)
		{
			return;
		}

		// Cache current stream detection, it could change while we are applying it
		FIntPoint CachedStreamResolution;
		{
			FScopeLock Lock(&StreamResolutionCriticalSection);
			CachedStreamResolution = StreamResolution;
		}

		if (VideoTrackFormat.Dim != CachedStreamResolution)
		{
			UE_LOGF(LogRivermaxMedia, Log, "Player needs to apply newly detected stream resolution : %dx%d", CachedStreamResolution.X, CachedStreamResolution.Y);

			{
				WaitForPendingTasks();

				AllocateBuffers(CachedStreamResolution);

				VideoTrackFormat.Dim = CachedStreamResolution;
			}

		}

		TickTimeManagement();
	}

	/* FRivermaxMediaPlayer implementation
	 *****************************************************************************/
	bool FRivermaxMediaPlayer::IsHardwareReady() const
	{
		return (RivermaxThreadNewState == EMediaState::Playing) || (RivermaxThreadNewState == EMediaState::Paused);
	}

	void FRivermaxMediaPlayer::SetupSampleChannels()
	{
		FMediaIOSamplingSettings VideoSettings = BaseSettings;
		VideoSettings.BufferSize = kMaxNumVideoFrameBuffer;

		// TODO: Initialize Audio, Anc buffers
		Samples->InitializeVideoBuffer(VideoSettings);
	}

	TSharedPtr<FMediaIOCoreTextureSampleConverter> FRivermaxMediaPlayer::CreateTextureSampleConverter() const
	{
		return MakeShared<FRivermaxMediaTextureSampleConverter>();
	}

	TSharedPtr<FMediaIOCoreTextureSampleBase> FRivermaxMediaPlayer::AcquireTextureSample_AnyThread() const
	{

		if ((Samples->NumVideoSamples() > 0 || IsJustInTimeRenderingEnabled()) && ProxySampleDummy.IsValid())
		{
			FScopeLock Lock(&ProxySampleAccessCriticalSection);
			// Create a copy of the proxy sample as Media Texture uses raw pointers for converters on Render thread and converter is set on game thread.
			TSharedPtr<FRivermaxMediaTextureSample> SampleToReturn = MakeShared<FRivermaxMediaTextureSample>();
			SampleToReturn->CopyConfiguration(ProxySampleDummy);
			return SampleToReturn;
		}
		else
		{
			return nullptr;
		}
	}

#if WITH_EDITOR
	FString FRivermaxMediaPlayer::GetAnalyticsEventPrefix() const
	{
		return TEXT("MediaFramework.Rivermax");
	}

	void FRivermaxMediaPlayer::GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const
	{
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionWidth"), FString::Printf(TEXT("%d"), StreamResolution.X)));
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionHeight"), FString::Printf(TEXT("%d"), StreamResolution.Y)));
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("FrameRate"), *VideoFrameRate.ToPrettyText().ToString()));
	}
#endif

	void FRivermaxMediaPlayer::OnInitializationCompleted(const FRivermaxInputInitializationResult& Result)
	{
		RivermaxThreadNewState = Result.bHasSucceed ? EMediaState::Playing : EMediaState::Error;
		bStreamSupportsGPUDirect = Result.bIsGPUDirectSupported;
	}

	bool FRivermaxMediaPlayer::ConfigureStream(const IMediaOptions* Options)
	{
		using namespace UE::RivermaxCore;

		// Resolve interface address
		IRivermaxCoreModule* Module = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
		if (Module == nullptr)
		{
			return false;
		}

		const FString DesiredInterface = Options->GetMediaOption(RivermaxMediaOption::InterfaceAddress, FString());
		const bool bFoundDevice = Module->GetRivermaxManager()->GetMatchingDevice(DesiredInterface, StreamOptions.InterfaceAddress);
		if (bFoundDevice == false)
		{
			UE_LOGF(LogRivermaxMedia, Error, "Could not find a matching interface for IP '%ls'", *DesiredInterface);
			return false;
		}

		StreamOptions.StreamAddress = Options->GetMediaOption(RivermaxMediaOption::StreamAddress, FString());
		StreamOptions.Port = Options->GetMediaOption(RivermaxMediaOption::Port, (int64)0);
		StreamOptions.bUseGPUDirect = Options->GetMediaOption(RivermaxMediaOption::UseGPUDirect, false);
		StreamOptions.FrameRate = VideoFrameRate;
		StreamOptions.PixelFormat = UE::RivermaxMediaUtils::Private::PixelFormatToRivermaxSamplingType(DesiredPixelFormat);
		const FVideoFormatInfo FormatInfo = FStandardVideoFormat::GetVideoFormatInfo(StreamOptions.PixelFormat);
		const uint32 PixelAlignment = FormatInfo.PixelGroupCoverage;
		const uint32 AlignedHorizontalResolution = (VideoTrackFormat.Dim.X % PixelAlignment) ? VideoTrackFormat.Dim.X + (PixelAlignment - (VideoTrackFormat.Dim.X % PixelAlignment)) : VideoTrackFormat.Dim.X;
		StreamOptions.EnforcedResolution = FIntPoint(AlignedHorizontalResolution, VideoTrackFormat.Dim.Y);
		StreamOptions.bEnforceVideoFormat = !bFollowsStreamResolution;

		return true;
	}

	void FRivermaxMediaPlayer::AllocateBuffers(const FIntPoint& InResolution)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxMediaPlayer::AllocateBuffers);
		using namespace UE::RivermaxCore;
		using namespace UE::RivermaxMediaUtils::Private;

		Samples->FlushSamples();

		// Only need to store latest frame and the number of frames delayed by.
		Samples->SetSampleBufferSize(FrameDelay + 1);

		if (VideoTextureSamplePool.IsValid())
		{
			VideoTextureSamplePool->Reset();
		}

		VideoTextureSamplePool = MakeUnique<FRivermaxMediaTextureSamplePool>();

		TSharedPtr<FRivermaxMediaTextureSample> NewSample;

		// Used to temporarily store initialized gpu buffers to avoid them returning to the pool before everything is initialized.
		TArray<TSharedPtr<FRivermaxMediaTextureSample>> TempStorage;
		// Allocate our pool of samples where incoming ones will be written and chosen from
		for (int32 Index = 0; Index < kMaxNumVideoFrameBuffer; Index++)
		{
			NewSample = VideoTextureSamplePool->AcquireShared(true /*Allocate New*/);
			NewSample->InitializeGPUBuffer(InResolution, DesiredPixelFormat, bStreamSupportsGPUDirect);
			NewSample->SampleConversionFence = RHICreateGPUFence(*FString::Printf(TEXT("RmaxConversionDoneFence_%02d"), Index));

			NewSample->SetReceptionState(IRivermaxSample::ESampleState::Idle);
			NewSample->SetInputFormat(DesiredPixelFormat);
			if (IS_FRAMELOCKED())
			{
				FrameLockedSamples[Index] = NewSample;
			}
			else
			{
				TempStorage.Add(NewSample);
			}
		}

		// Return initialized buffers back to the pool
		TempStorage.Empty();

		// Create the proxy sample that is going to be used for color conversion.
		ENQUEUE_RENDER_COMMAND(FRivermaxMediaTextureSample)(
			[NewSample, Resolution = InResolution, this](FRHICommandListImmediate& RHICmdList)
			{
				FScopeLock Lock(&ProxySampleAccessCriticalSection);

				ProxySampleDummy = MakeShared<FRivermaxMediaTextureSample>();
				ProxySampleDummy->SetTexture(CreateIntermediateRenderTarget(RHICmdList, Resolution, NewSample->GetPixelFormat(), NewSample->IsOutputSrgb()));
				ProxySampleDummy->SetProperties(NewSample->GetStride(), VideoTrackFormat.Dim.X, VideoTrackFormat.Dim.Y, NewSample->GetFormat(), FTimespan(0), FFrameRate(), FTimecode(), SourceColorSettings);
			});


		// Allocation is done on render thread so let's make sure it's completed before pursuing
		FRenderCommandFence RenderFence;
		RenderFence.BeginFence();
		RenderFence.Wait();
		VideoTextureSamplePool->Tick();

	}

	void FRivermaxMediaPlayer::OnStreamError()
	{
		// If the stream ends up in error, stop the player
		UE_LOGF(LogRivermaxMedia, Error, "Stream caught an error. Player will stop.");
		RivermaxThreadNewState = EMediaState::Error;
	}

	TRefCountPtr<FRHITexture> FRivermaxMediaPlayer::CreateIntermediateRenderTarget(FRHICommandListImmediate& RHICmdList, const FIntPoint& InDim, EPixelFormat InPixelFormat, bool bInSRGB)
	{
		TRefCountPtr<FRHITexture> TextureToReturn;
		// create output render target if necessary
		ETextureCreateFlags OutputCreateFlags = (bInSRGB ? TexCreate_SRGB : TexCreate_None) | TexCreate_UAV;
		OutputCreateFlags |= TexCreate_UAV;
		OutputCreateFlags |= ETextureCreateFlags::RenderTargetable;


		const static FLazyName ClassName(TEXT("FRivermaxMediaTextureSample"));
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FRivermaxMediaTextureOutput"))
			.SetExtent(InDim)
			.SetFormat(InPixelFormat)
			//.SetNumMips(1)
			.SetFlags(OutputCreateFlags | ETextureCreateFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetClearValue(FClearValueBinding(FLinearColor::Red))
			.SetClassName(ClassName)
			.SetOwnerName(*GetMediaName().ToString());

		TextureToReturn = RHICmdList.CreateTexture(Desc);

		TextureToReturn->SetName(TEXT("RivermaxMediaTexture"));
		TextureToReturn->SetOwnerName(*GetMediaName().ToString());
		return MoveTemp(TextureToReturn);
	}

	bool FRivermaxMediaPlayer::JustInTimeSampleRender_RenderThread(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDestinationTexture, TSharedPtr<FMediaIOCoreTextureSampleBase>& InJITRProxySample)
	{
		// Player renders into an intermediate render target that is then used to be converted into the right color encoding if needed.
		InDestinationTexture = ProxySampleDummy->GetTexture();
		TRACE_CPUPROFILER_EVENT_SCOPE(RivermaxPlayerLateUpdate);
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Rmax::StartingRender %llu"), (GFrameCounterRenderThread)));

		// Sanity check to make sure that we are not rendering this sample twice per frame.
		check(LastEngineRTFrameThatUpdatedJustInTime != GFrameCounterRenderThread);
		LastEngineRTFrameThatUpdatedJustInTime = GFrameCounterRenderThread;

		FFrameInfo FrameInformation;
		FrameInformation.RequestedTimecode = InJITRProxySample->GetTimecode().Get(FTimecode());
		FrameInformation.SampleTimespan = InJITRProxySample->GetTime().Time;
		FrameInformation.EvaluationOffset = InJITRProxySample->GetEvaluationOffsetInSeconds();
		FrameInformation.FrameNumber = GFrameCounterRenderThread;

		TSharedPtr<FRivermaxMediaTextureSample> SamplePtr = StaticCastSharedPtr<FRivermaxMediaTextureSample>(PickSampleToRender_RenderThread(FrameInformation));
		if (!SamplePtr.IsValid())
		{
			UE_LOGF(LogRivermaxMedia, Verbose, "Couldn't find a sample to render for frame %u.", FrameInformation.FrameNumber);
			return false;
		}

		if (!InDestinationTexture.IsValid())
		{
			UE_LOGF(LogRivermaxMedia, Warning, "Couldn't find texture to render into for sample %llu.", SamplePtr->GetFrameNumber());
			return false;
		}

		// Verify if the frame we will use for rendering is still being rendered for the previous one.
		if (!SamplePtr->TryLockForRendering())
		{
			if (IS_FRAMELOCKED())
			{
				UE_LOGF(LogRivermaxMedia, Warning, "Framelocked sample %llu was still rendering when we expected to reuse its location.", SamplePtr->GetFrameNumber());
				return false;
			}
			else
			{
				UE_LOGF(LogRivermaxMedia, Verbose, "Sample %llu was either already rendered or is already being rendered.", SamplePtr->GetFrameNumber());
				return false;
			}
		}

		FSampleConverterOperationSetup ConverterSetup;
		SampleUploadSetupTaskThreadMode(SamplePtr, ConverterSetup);

		// If no input data was provided, no need to render
		if (ConverterSetup.GetGPUBufferFunc == nullptr)
		{
			ensureMsgf(false, TEXT("Rivermax player late update succeeded but didn't provide any source data."));
			return false;
		}

		FRDGBuilder GraphBuilder(RHICmdList);
		if (ConverterSetup.PreConvertFunc)
		{
			ConverterSetup.PreConvertFunc(GraphBuilder);
		}

		using namespace UE::RivermaxShaders;
		using namespace UE::RivermaxMediaUtils::Private;

		const FSourceBufferDesc SourceBufferDesc = GetBufferDescription(VideoTrackFormat.Dim, SamplePtr->GetInputFormat());
		{
			FRDGBufferRef InputBuffer;

			RDG_EVENT_SCOPE_STAT(GraphBuilder, RivermaxSource_SampleConversion, "RivermaxSource_SampleConversion");
			SCOPED_DRAW_EVENT(GraphBuilder.RHICmdList, Rivermax_SampleConverter);

			FRDGTextureRef OutputResource = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InDestinationTexture, TEXT("RivermaxMediaTextureOutputResource")));

			InputBuffer = GraphBuilder.RegisterExternalBuffer(ConverterSetup.GetGPUBufferFunc(), TEXT("RMaxGPUBuffer"));

			const FIntPoint ProcessedOutputDimension = { (int32)SourceBufferDesc.NumberOfElements, 1 };
			const FIntVector GroupCount = UE::RivermaxMediaUtils::Private::GetComputeShaderGroupCount(SourceBufferDesc.NumberOfElements);
			FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

			//Configure shader and add conversion pass based on desired pixel format
			switch (SamplePtr->GetInputFormat())
			{
			case ERivermaxPixelFormat::YUV422_8bit:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RivermaxSampleConverter::YUV8ShaderSetup);

				const FMatrix YUVToRGBMatrix = SamplePtr->GetYUVToRGBMatrix();
				const FVector YUVOffset(MediaShaders::YUVOffset8bits);
				TShaderMapRef<FYUV8Bit422ToRGBACS> ComputeShader(GlobalShaderMap);
				FYUV8Bit422ToRGBACS::FParameters* Parameters = ComputeShader->AllocateAndSetParameters(GraphBuilder, InputBuffer, OutputResource, YUVToRGBMatrix, YUVOffset, ProcessedOutputDimension.X, ProcessedOutputDimension.Y);

				FComputeShaderUtils::AddPass(
					GraphBuilder
					, RDG_EVENT_NAME("YUV8Bit422ToRGBA")
					, ComputeShader
					, Parameters
					, GroupCount);
				break;
			}
			case ERivermaxPixelFormat::YUV422_10bit:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RivermaxSampleConverter::YUV10ShaderSetup);

				const FMatrix YUVToRGBMatrix = SamplePtr->GetYUVToRGBMatrix();
				const FVector YUVOffset(MediaShaders::YUVOffset10bits);
				TShaderMapRef<FYUV10Bit422ToRGBACS> ComputeShader(GlobalShaderMap);
				FYUV10Bit422ToRGBACS::FParameters* Parameters = ComputeShader->AllocateAndSetParameters(GraphBuilder, InputBuffer, OutputResource, YUVToRGBMatrix, YUVOffset, ProcessedOutputDimension.X, ProcessedOutputDimension.Y);

				FComputeShaderUtils::AddPass(
					GraphBuilder
					, RDG_EVENT_NAME("YUV10Bit422ToRGBA")
					, ComputeShader
					, Parameters
					, GroupCount);
				break;
			}
			case ERivermaxPixelFormat::RGB_8bit:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RivermaxSampleConverter::RGB8ShaderSetup);

				TShaderMapRef<FRGB8BitToRGBA8CS> ComputeShader(GlobalShaderMap);
				FRGB8BitToRGBA8CS::FParameters* Parameters = ComputeShader->AllocateAndSetParameters(GraphBuilder, InputBuffer, OutputResource, ProcessedOutputDimension.X, ProcessedOutputDimension.Y);

				FComputeShaderUtils::AddPass(
					GraphBuilder
					, RDG_EVENT_NAME("RGB8BitToRGBA8")
					, ComputeShader
					, Parameters
					, GroupCount);
				break;
			}
			case ERivermaxPixelFormat::RGB_10bit:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RivermaxSampleConverter::RGB10ShaderSetup);

				TShaderMapRef<FRGB10BitToRGBA10CS> ComputeShader(GlobalShaderMap);
				FRGB10BitToRGBA10CS::FParameters* Parameters = ComputeShader->AllocateAndSetParameters(GraphBuilder, InputBuffer, OutputResource, ProcessedOutputDimension.X, ProcessedOutputDimension.Y);

				FComputeShaderUtils::AddPass(
					GraphBuilder
					, RDG_EVENT_NAME("RGB10BitToRGBA")
					, ComputeShader
					, Parameters
					, GroupCount);
				break;
			}
			case ERivermaxPixelFormat::RGB_12bit:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RivermaxSampleConverter::RGB12ShaderSetup);

				TShaderMapRef<FRGB12BitToRGBA12CS> ComputeShader(GlobalShaderMap);
				FRGB12BitToRGBA12CS::FParameters* Parameters = ComputeShader->AllocateAndSetParameters(GraphBuilder, InputBuffer, OutputResource, ProcessedOutputDimension.X, ProcessedOutputDimension.Y);

				FComputeShaderUtils::AddPass(
					GraphBuilder
					, RDG_EVENT_NAME("RGB12BitToRGBA")
					, ComputeShader
					, Parameters
					, GroupCount);
				break;
			}
			case ERivermaxPixelFormat::RGB_16bit_Float:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RivermaxSampleConverter::RGB16FloatShaderSetup);

				TShaderMapRef<FRGB16fBitToRGBA16fCS> ComputeShader(GlobalShaderMap);
				FRGB16fBitToRGBA16fCS::FParameters* Parameters = ComputeShader->AllocateAndSetParameters(GraphBuilder, InputBuffer, OutputResource, ProcessedOutputDimension.X, ProcessedOutputDimension.Y);

				FComputeShaderUtils::AddPass(
					GraphBuilder
					, RDG_EVENT_NAME("RGB16fBitToRGBA")
					, ComputeShader
					, Parameters
					, GroupCount);
				break;
			}
			default:
			{
				ensureMsgf(false, TEXT("Unhandled pixel format (%d) given to Rivermax MediaSample converter"), EnumToUnderlyingType(SamplePtr->GetInputFormat()));
				return false;
			}
			}
		}

		if (ConverterSetup.PostConvertFunc)
		{
			ConverterSetup.PostConvertFunc(GraphBuilder);
		}

		GraphBuilder.Execute();
		return true;
	}

	TSharedPtr<FMediaIOCoreTextureSampleBase> FRivermaxMediaPlayer::PickSampleToRenderFramelocked_RenderThread
	(const FFrameInfo& InFrameInformation)
	{
		TSharedPtr<FRivermaxMediaTextureSample> Sample = FrameLockedSamples[TO_EXPECTED_SAMPLE_INDEX(InFrameInformation.FrameNumber)];

		return Sample;
	}

	TSharedPtr<FMediaIOCoreTextureSampleBase> FRivermaxMediaPlayer::PickSampleToRenderForTimeSynchronized_RenderThread(const FFrameInfo& InFrameInformation)
	{
		// Determine the target reference time.
		const bool bTimecodeTarget = (EvaluationType == EMediaIOSampleEvaluationType::Timecode && InFrameInformation.RequestedTimecode != FTimecode());

		FTimespan TargetSampleTimespan;
		if (bTimecodeTarget)
		{
			TargetSampleTimespan = InFrameInformation.RequestedTimecode.ToTimespan(VideoFrameRate);
		}
		else
		{
			TargetSampleTimespan = InFrameInformation.SampleTimespan;
		}

		// Apply latency / evaluation-offset correction (encodes FrameDelay and any user-configured offset).  Both target types go through this.
		const FTimespan TargetTimespanCorrected = TargetSampleTimespan - FTimespan::FromSeconds(InFrameInformation.EvaluationOffset);

		// Get all available video samples
		const TArray<TSharedPtr<IMediaTextureSample>> TextureSamples = Samples->GetVideoSamples();
		if (TextureSamples.Num() == 0)
		{
			return nullptr;
		}

		// One frame duration used to bound the exact-hit window for ANC-timecode matching.
		const FTimespan OneFrameDuration = FTimespan::FromSeconds(1.0 / VideoFrameRate.AsDecimal());

		int32 ClosestIndex    = -1;
		int64 SmallestInterval = TNumericLimits<int64>::Max();

		for (int32 Index = 0; Index < TextureSamples.Num(); ++Index)
		{
			TSharedPtr<FRivermaxMediaTextureSample> Sample =
			    StaticCastSharedPtr<FRivermaxMediaTextureSample>(TextureSamples[Index]);

			// Select the sample's reference timespan:
			//
			//   ANC timecode path (preferred when available):
			//     Converts the sender-stamped timecode to a midnight based timespan.
			//
			//   Reception time fallback:
			//     PTP window recorded when the frame arrived.  Used when no ANC stream is configured / ANC hasn't arrived yet. In a PTP locked setup this is a good approximation of the timecode path.
			FTimespan SampleRefTime;
			FTimespan SampleWindowEnd;

			const TOptional<FTimecode> SampleTimecode = Sample->GetTimecode();
			if (bTimecodeTarget && SampleTimecode.IsSet())
			{
				SampleRefTime   = SampleTimecode.GetValue().ToTimespan(VideoFrameRate);
				SampleWindowEnd = SampleRefTime + OneFrameDuration;
			}
			else if (!bTimecodeTarget)
			{
				SampleRefTime   = Sample->FrameReceptionStart;
				SampleWindowEnd = Sample->FrameReceptionEnd;
			}
			else
			{
				// Timecode mode but ANC hasn't arrived for this sample yet — skip rather
				// than compare a timecode-based target against a PTP wall-clock timestamp.
				continue;
			}

			// Exact window hit: the corrected target falls squarely within this frame.
			if (TargetTimespanCorrected >= SampleRefTime && TargetTimespanCorrected < SampleWindowEnd)
			{
				ClosestIndex = Index;
				break;
			}

			const int64 TestInterval = FMath::Abs((SampleRefTime - TargetTimespanCorrected).GetTicks());
			if (TestInterval <= SmallestInterval)
			{
				ClosestIndex = Index;
				SmallestInterval = TestInterval;
			}
			else
			{
				// Since our samples are stored in chronological order, it makes no sense
				// to continue searching. The interval will continue increasing.
				break;
			}
		}

		// When in timecode mode but none of the buffered samples had ANC timecode (e.g., no ANC stream
		// configured or ANC stream stalled), fall back to a reception-time comparison so the caller still
		// gets the closest available frame rather than nothing.
		if (bTimecodeTarget && ClosestIndex < 0)
		{
			const FTimespan FallbackTargetCorrected = InFrameInformation.SampleTimespan - FTimespan::FromSeconds(InFrameInformation.EvaluationOffset);
			SmallestInterval = TNumericLimits<int64>::Max();

			for (int32 Index = 0; Index < TextureSamples.Num(); ++Index)
			{
				TSharedPtr<FRivermaxMediaTextureSample> FallbackSample =
				    StaticCastSharedPtr<FRivermaxMediaTextureSample>(TextureSamples[Index]);

				const FTimespan SampleRefTime   = FallbackSample->FrameReceptionStart;
				const FTimespan SampleWindowEnd = FallbackSample->FrameReceptionEnd;

				if (FallbackTargetCorrected >= SampleRefTime && FallbackTargetCorrected < SampleWindowEnd)
				{
					ClosestIndex = Index;
					break;
				}

				const int64 TestInterval = FMath::Abs((SampleRefTime - FallbackTargetCorrected).GetTicks());
				if (TestInterval <= SmallestInterval)
				{
					ClosestIndex = Index;
					SmallestInterval = TestInterval;
				}
				else
				{
					break;
				}
			}
		}

		if (ClosestIndex < 0)
		{
			return nullptr;
		}
		return StaticCastSharedPtr<FMediaIOCoreTextureSampleBase, IMediaTextureSample, ESPMode::ThreadSafe>(TextureSamples[ClosestIndex]);
	}

	void FRivermaxMediaPlayer::PostSampleUsage(FRDGBuilder& GraphBuilder, TSharedPtr<FRivermaxMediaTextureSample> Sample)
	{
		GraphBuilder.AddPass(RDG_EVENT_NAME("RivermaxPostSampleUsage"),
			ERDGPassFlags::NeverCull,
			[SamplePtr = Sample, this](FRHICommandList& RHICmdList)
			{
				RHI_BREADCRUMB_EVENT_STAT(RHICmdList, RivermaxMedia_SampleUsageFence, "RivermaxMedia_SampleUsageFence");

				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Rmax::ReleasingSample %llu"), (SamplePtr->GetFrameNumber())));

				// Write a fence in the post sample usage pass to be able to know when we can reuse it
				RHICmdList.WriteGPUFence(SamplePtr->SampleConversionFence);

				++TasksInFlight;

				// This async task is waiting for the GPU to be finished with Sample's resources and then release them.
				UE::Tasks::Launch(UE_SOURCE_LOCATION,
					[SamplePtr, this]()
					{
						ON_SCOPE_EXIT
						{
							--TasksInFlight;
						};

						TRACE_CPUPROFILER_EVENT_SCOPE(RmaxWaitForShader);
						do
						{
							const bool bHasValidFence = SamplePtr->SampleConversionFence.IsValid();
							const bool bHasFenceCompleted = bHasValidFence ? SamplePtr->SampleConversionFence->Poll() : false;
							if (bHasValidFence == false || bHasFenceCompleted)
							{
								break;
							}

							FPlatformProcess::SleepNoStats(static_cast<float>(SleepTimeSeconds));

						} while (true);

						// We clear the sample states, mark this sample as ready for reuse and that rendering is completed.
						SamplePtr->ShutdownPoolable();
					});
			});
	}

	void FRivermaxMediaPlayer::OnVideoFrameReceptionError(TSharedPtr<IRivermaxVideoSample> InVideoFrameSample)
	{
		TSharedPtr<FRivermaxMediaTextureSample> Sample = StaticCastSharedPtr<FRivermaxMediaTextureSample>(InVideoFrameSample);
		if (!Sample.IsValid())
		{
			return;
		}

		UE_LOGF(LogRivermaxMedia, Warning, "Issue receiving frame number %llu.", Sample->GetFrameNumber());
		if (Sample->IsBeingRendered())
		{
			Sample->SetReceptionState(IRivermaxSample::ESampleState::ReceptionError);
			Sample->GetSampleReceivedEvent()->Trigger();
		}
		else
		{
			Sample->SetReceptionState(IRivermaxSample::ESampleState::Idle);
		}
	}

	void FRivermaxMediaPlayer::OnVideoFormatChanged(const FRivermaxInputVideoFormatChangedInfo& NewFormatInfo)
	{
		const ERivermaxPixelFormat NewFormat = UE::RivermaxMediaUtils::Private::RivermaxSamplingTypeToPixelFormat(NewFormatInfo.PixelFormat);
		const FIntPoint NewResolution = { (int32)NewFormatInfo.Width, (int32)NewFormatInfo.Height };
		bool bNeedReinitializing = (NewFormatInfo.PixelFormat != StreamOptions.PixelFormat);
		bNeedReinitializing |= (NewFormatInfo.Width != VideoTrackFormat.Dim.X || NewFormatInfo.Height != VideoTrackFormat.Dim.Y);

		UE_LOGF(LogRivermaxMedia, Log, "New video format detected: %dx%d with pixel format '%ls'", NewResolution.X, NewResolution.Y, *UEnum::GetValueAsString(NewFormat));

		if (bNeedReinitializing && bFollowsStreamResolution)
		{
			FScopeLock Lock(&StreamResolutionCriticalSection);
			StreamResolution = NewResolution;
		}
	}

	TSharedPtr<IRivermaxAncSample> FRivermaxMediaPlayer::HandleAncFrameRequested(const FRivermaxInputAncFrameDescriptor& FrameInfo)
	{
		TSharedPtr<FRivermaxAncSample> Sample = MakeShared<FRivermaxAncSample>();
		Sample->RTPTimestamp = FrameInfo.Timestamp;
		return Sample;
	}

	void FRivermaxMediaPlayer::HandleAncFrameReceived(TSharedPtr<IRivermaxAncSample> InReceivedAncFrame)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::AncFrameReceived);
		if (!InReceivedAncFrame)
		{
			return;
		}

		TSharedPtr<FRivermaxAncSample> AncSample = StaticCastSharedPtr<FRivermaxAncSample>(InReceivedAncFrame);

		using namespace UE::RivermaxMediaUtils::Private;
		if ((AncSample->GetDID() & 0xFF) == AncTimecodeDID && (AncSample->GetSDID() & 0xFF) == AncTimecodeSDID)
		{
			const FTimecode DecodedTimecode = UE::RivermaxCore::AtcUDW10ToTimecode(AncSample->GetUserDataWords());

			// Store the decoded timecode keyed by RTP timestamp so it can be matched to the video sample that shares the same timestamp (same 90 kHz PTP clock).
			{
				FScopeLock Lock(&PendingAncDataLock);
				PendingAncData.FindOrAdd(AncSample->RTPTimestamp).Timecode = DecodedTimecode;

				const int32 MaxPendingEntries = kMaxNumVideoFrameBuffer + 2;
				if (PendingAncData.Num() > MaxPendingEntries)
				{
					// Evict by smallest key value as a proxy for "oldest entry". This is incorrect after
					// the 32-bit RTP timestamp wraps (~13.3 hours at 90 kHz): wrapped (small) values are
					// newest, so the wrong entry gets dropped. In practice the map rarely exceeds one or
					// two entries (ANC is always received before the corresponding video frame completes),
					// so the eviction path almost never fires. If it does fire across a wraparound, at most
					// kMaxNumVideoFrameBuffer frames will lose their timecode before the map drains and
					// resumes normal operation.
					uint32 OldestKey = TNumericLimits<uint32>::Max();
					for (const auto& Pair : PendingAncData)
					{
						if (Pair.Key < OldestKey)
						{
							OldestKey = Pair.Key;
						}
					}
					PendingAncData.Remove(OldestKey);
				}
			}

			UE_LOG(LogRivermaxMedia, VeryVerbose, TEXT("ANC Timecode ts=%u: %02d:%02d:%02d:%02d%s"), AncSample->RTPTimestamp,
				DecodedTimecode.Hours, DecodedTimecode.Minutes, DecodedTimecode.Seconds, DecodedTimecode.Frames,
				DecodedTimecode.bDropFrameFormat ? TEXT(";") : TEXT(":"));
		}
	}

	void FRivermaxMediaPlayer::HandleAncFrameReceptionError()
	{
		UE_LOG(LogRivermaxMedia, Verbose, TEXT("ANC frame reception error."));
	}

	bool FRivermaxMediaPlayer::IsReadyToPlay() const
	{
		if (RivermaxThreadNewState == EMediaState::Playing)
		{
			FScopeLock Lock(&StreamResolutionCriticalSection);
			return StreamResolution == VideoTrackFormat.Dim;
		}

		return false;
	}

	void FRivermaxMediaPlayer::WaitForPendingTasks()
	{
		// Flush any rendering activity to be sure we can move on with clearing resources. 
		FlushRenderingCommands();

		// Wait for all pending tasks to complete. They should all complete at some point but add a timeout as a last resort. 
		constexpr double TimeoutSeconds = 2.0;
		const double StartTimeSeconds = FPlatformTime::Seconds();
		while (TasksInFlight > 0)
		{
			FPlatformProcess::SleepNoStats(static_cast<float>(SleepTimeSeconds));
			if ((FPlatformTime::Seconds() - StartTimeSeconds) > TimeoutSeconds)
			{
				UE_LOGF(LogRivermaxMedia, Warning, "Timed out waiting for pendings tasks to finish.");
				break;
			}
		}
	}

	bool FRivermaxMediaPlayer::WaitForSample(const TSharedPtr<FRivermaxMediaTextureSample>& Sample, const uint64 AwaitingFrameNumber, FWaitConditionFunc WaitConditionFunction, const double TimeoutSeconds)
	{
		const double StartTimeSeconds = FPlatformTime::Seconds();

		while (true)
		{
			// Our goal here is to wait until the expected frame is available to be used (received) unless there is a timeout
			if (WaitConditionFunction(Sample))
			{
				return true;
			}

			FPlatformProcess::SleepNoStats(static_cast<float>(SleepTimeSeconds));

			{
				if (((FPlatformTime::Seconds() - StartTimeSeconds) > TimeoutSeconds))
				{
					UE_LOGF(LogRivermaxMedia, Error, "Timed out waiting for frame %llu.", AwaitingFrameNumber);
					return false;
				}
			}
		}
	}

	void FRivermaxMediaPlayer::SampleUploadSetupTaskThreadMode(TSharedPtr<FRivermaxMediaTextureSample> Sample, FSampleConverterOperationSetup& OutConverterSetup)
	{
		// We will always be providing a buffer already available to the GPU even when not using gpudirect
		// Once a frame has arrived on system, we will upload it to the allocated gpu buffer.
		OutConverterSetup.GetGPUBufferFunc = [SamplePtr = Sample]() { return SamplePtr->GetGPUBuffer(); };

		const uint64 NextFrameExpectations = (IS_FRAMELOCKED()) ? TO_EXPECTED_SAMPLE_FRAME_NUM(GFrameCounterRenderThread) : Sample->GetFrameNumber();

		OutConverterSetup.PreConvertFunc = [SamplePtr = Sample, NextFrameExpectations, this](const FRDGBuilder& GraphBuilder)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Rmax::01_PreConvertFunction %llu"), NextFrameExpectations));

				SamplePtr->SetAwaitingForGPUTransfer(true);

				// When GPUDirect is not involved, we have an extra step to do. We need to wait for the sample to be received
				// but also initiate the memcopy to gpu memory for it to be rendered
				if (bStreamSupportsGPUDirect == false)
				{
					constexpr uint32 Offset = 0;
					const uint32 Size = SamplePtr->GetGPUBuffer()->GetSize();


					++TasksInFlight;
					UE::Tasks::Launch(UE_SOURCE_LOCATION,
						[Size, SamplePtr, NextFrameExpectations, this]()
						{
							ON_SCOPE_EXIT
							{
								--TasksInFlight;
							};
							TRACE_CPUPROFILER_EVENT_SCOPE(RmaxWaitAndCopyPixels);
							TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Rmax::WaitAndCopyPixels %llu"), (NextFrameExpectations)));

							// Indicates if we sample reception started on the input thread.
							bool bStreamReceptionStarted = true;

							// In frame locked mode the packets potentially haven't started being received yet. 
							// This waits for frame locked samples to start writing packets and then for the completion of the reception.
							if (IS_FRAMELOCKED())
							{
								TRACE_CPUPROFILER_EVENT_SCOPE(RmaxWaitForPixels);

								// Waiting for reception.
								const double TimeoutSeconds = CVarRivermaxDefaultThreadBlockTimeoutSec.GetValueOnAnyThread();
								const double StartTimeSeconds = FPlatformTime::Seconds();

								const double OneFrameTime = 1. / GetFrameRate().AsDecimal();

								// This loop will continue until it times out or the frame number matches or exceeds the expected frame number and the sample has been received or errored out.
								while (SamplePtr->GetFrameNumber() < NextFrameExpectations ||
									(SamplePtr->GetReceptionState() != IRivermaxSample::ESampleState::Received && SamplePtr->GetReceptionState() != IRivermaxSample::ESampleState::ReceptionError))
								{
									if ((FPlatformTime::Seconds() - StartTimeSeconds) > TimeoutSeconds)
									{
										UE_LOGF(LogRivermaxMedia, Warning, "Timed out waiting for frame #%llu to start being received.", NextFrameExpectations);
										bStreamReceptionStarted = false;
										break;
									}

									// This could turn into active loop only if frame number doesn't match and the sample is signaled.
									// However such a case can only happen if the sample was received but wasn't rendered previously.
									SamplePtr->GetSampleReceivedEvent()->Wait(FTimespan::FromSeconds(OneFrameTime));
								}
							}

							if (SamplePtr->GetFrameNumber() != NextFrameExpectations)
							{
								UE_LOGF(LogRivermaxMedia, Warning, "Rendering unexpected frame %llu, when frame %llu was expected.", SamplePtr->GetFrameNumber(), NextFrameExpectations);
							}

							// In case reception failed mid way.
							if (!bStreamReceptionStarted || SamplePtr->GetReceptionState() == IRivermaxSample::ESampleState::ReceptionError)
							{
								UE_LOGF(LogRivermaxMedia, Warning, "Incomplete or failed pixels will be rendered for frame %llu", NextFrameExpectations);
							}

							// Set this for debugging purposes to know exactly when this sample is shifted to rendering state.
							SamplePtr->SetReceptionState(IRivermaxSample::ESampleState::Rendering);

							// Signals the RHI thread that the GPU transfer has completed.
							SamplePtr->SetAwaitingForGPUTransfer(false);
						}
					);
				}

				// Stream supports GPU Direct
				else
				{
					++TasksInFlight;
					UE::Tasks::Launch(UE_SOURCE_LOCATION,
						[SamplePtr, NextFrameExpectations, this]()
						{
							ON_SCOPE_EXIT
							{
								--TasksInFlight;
							};

							TRACE_CPUPROFILER_EVENT_SCOPE(RmaxWaitSampleReceptionGPUDirect);
							TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Rmax::02_WaitSampleReceivedGpuDirect %llu"), (NextFrameExpectations)));

							double StartTime = FPlatformTime::Seconds();
							// In frame locked mode the packets potentially haven't started being received yet. 
							// This waits for frame locked samples to start writing packets and then for the completion of the reception.
							if (IS_FRAMELOCKED())
							{
								// Waiting for reception.
								const double TimeoutSeconds = CVarRivermaxDefaultThreadBlockTimeoutSec.GetValueOnAnyThread();
								const double StartTimeSeconds = FPlatformTime::Seconds();

								const double OneFrameTime = 1. / GetFrameRate().AsDecimal();

								// This loop will continue until it times out or the frame number matches or exceeds the expected frame number and the sample has been received or errored out.
								while (SamplePtr->GetFrameNumber() < NextFrameExpectations || 
									(SamplePtr->GetReceptionState() != IRivermaxSample::ESampleState::Received && SamplePtr->GetReceptionState() != IRivermaxSample::ESampleState::ReceptionError))
								{
									if ((FPlatformTime::Seconds() - StartTimeSeconds) > TimeoutSeconds)
									{
										UE_LOGF(LogRivermaxMedia, Warning, "Timed out waiting for frame #%llu to start being received.", NextFrameExpectations);
										break;
									}

									// This could turn into active loop if frame number doesn't match, but the sample is signaled.
									// However such a case can only happen if the sample was received but wasn't rendered previously.
									SamplePtr->GetSampleReceivedEvent()->Wait(FTimespan::FromSeconds(OneFrameTime));
								}
							}

							if (SamplePtr->GetFrameNumber() != NextFrameExpectations)
							{
								double Elapsed = FPlatformTime::Seconds() - StartTime;
								const TCHAR*& SampleStateString = FRivermaxMediaTextureSample::SampleStateToStringRef(SamplePtr->GetReceptionState());

								UE_LOGF(LogRivermaxMedia, Warning, "1. Rendering unexpected frame %llu, when frame %llu was expected. Elapsed wait time %f. State: %ls", SamplePtr->GetFrameNumber(), NextFrameExpectations, Elapsed, SampleStateString);
							}

							// In case reception failed mid way.
							if (SamplePtr->GetReceptionState() == IRivermaxSample::ESampleState::ReceptionError)
							{
								UE_LOGF(LogRivermaxMedia, Warning, "Incomplete or failed pixels will be rendered for frame %llu", NextFrameExpectations);
							}

							// Set this for debugging purposes to know exactly when this sample is shifted to rendering state.
							SamplePtr->SetReceptionState(IRivermaxSample::ESampleState::Rendering);

							// Signals the RHI thread that the GPU transfer has completed.
							SamplePtr->SetAwaitingForGPUTransfer(false);
						}
					);
				}



				RHI_BREADCRUMB_EVENT_STAT(GraphBuilder.RHICmdList, Rmax_WaitForPixels, "Rmax::WaitForPixels");

				// Since we are going to enqueue a lambda that can potentially sleep in the RHI thread if the pixels haven't arrived,
				// we dispatch the existing commands (including the draw event start timing in the SCOPED_DRAW_EVENT above) before any potential sleep.
				GraphBuilder.RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);

				// Setup requirements for sample to be ready to be rendered
				GraphBuilder.RHICmdList.EnqueueLambda(
					[SamplePtr, NextFrameExpectations, this](FRHICommandListImmediate&)
					{
						FWaitConditionFunc ReadyToRenderConditionFunc = [](const TSharedPtr<FRivermaxMediaTextureSample>& Sample)
							{
								return Sample->IsAwaitingForGPUTransfer() == false;
							};

						TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Rmax::04_WaitForUploadToFinish %llu"), (NextFrameExpectations)));

						// Set timeout to 10 seconds which should be enough to make sure that the frame is received or errors out.
						const double TimeoutSeconds = 10.;
						WaitForSample(SamplePtr, NextFrameExpectations, MoveTemp(ReadyToRenderConditionFunc), TimeoutSeconds);

						if (SamplePtr->GetFrameNumber() != NextFrameExpectations)
						{
							UE_LOGF(LogRivermaxMedia, Warning, "2. Rendering unexpected frame %llu, when frame %llu was expected.", SamplePtr->GetFrameNumber(), NextFrameExpectations);
						}

					}
				);

			};

		// Setup post sample usage pass 
		OutConverterSetup.PostConvertFunc = [SamplePtr = Sample, this](FRDGBuilder& GraphBuilder)
			{
				PostSampleUsage(GraphBuilder, SamplePtr);
			};
	}

}

#undef LOCTEXT_NAMESPACE
