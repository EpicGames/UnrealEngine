// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaIOCorePlayerBase.h"
#include "IRivermaxInputStream.h"
#include "RivermaxMediaTextureSampleConverter.h"
#include "RivermaxMediaTypes.h"
#include "RivermaxTypes.h"
#include "Templates/Function.h"

#include <atomic>

struct FSlateBrush;
class IMediaEventSink;

enum class EMediaTextureSampleFormat;
enum class EMediaIOSampleType;


namespace  UE::RivermaxMedia
{
	using namespace UE::RivermaxCore;
	
	class FRivermaxMediaTextureSample;
	class FRivermaxMediaTextureSamplePool;

	/**
	 * Implements a media player using rivermax.
	 */
	class FRivermaxMediaPlayer : public FMediaIOCorePlayerBase, public IRivermaxInputStreamListener
	{
		using Super = FMediaIOCorePlayerBase;
	public:

		/**
		 * Create and initialize a new instance.
		 *
		 * @param InEventSink The object that receives media events from this player.
		 */
		FRivermaxMediaPlayer(IMediaEventSink& InEventSink);

		/** Virtual destructor. */
		virtual ~FRivermaxMediaPlayer();

	public:
		//~ Begin FMediaIOCorePlayerBase interface

		/** Called by the sample converter to setup rendering commands to convert this sample into texture. */
		virtual bool JustInTimeSampleRender_RenderThread(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDestinationTexture, TSharedPtr<FMediaIOCoreTextureSampleBase>& JITRProxySample) override;
	protected:
		virtual TSharedPtr<FMediaIOCoreTextureSampleBase> PickSampleToRenderFramelocked_RenderThread(const FFrameInfo& InFrameInformation) override;
		
		/**  Pick sample based on Vsync timecode and Start and End of sample reception. */
		virtual TSharedPtr<FMediaIOCoreTextureSampleBase> PickSampleToRenderForTimeSynchronized_RenderThread(const FFrameInfo& InFrameInformation) override;
		//~ End FMediaIOCorePlayerBase interface

	public: 
		//~ Begin IMediaPlayer interface
		virtual void Close() override;
		virtual FGuid GetPlayerPluginGUID() const override;
		virtual bool Open(const FString& Url, const IMediaOptions* Options) override;
		virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override;
		//~ End IMediaPlayer interface

		//~ Begin ITimedDataInput interface
#if WITH_EDITOR
		virtual const FSlateBrush* GetDisplayIcon() const override;
#endif
		//~ End ITimedDataInput interface

		ERivermaxPixelFormat GetDesiredPixelFormat()
		{
			return DesiredPixelFormat;
		};

		//~ Begin IRivermaxInputStreamListener interface
		virtual void OnInitializationCompleted(const FRivermaxInputInitializationResult& Result) override;
		virtual TSharedPtr<IRivermaxVideoSample> OnVideoFrameRequested(const FRivermaxInputVideoFrameDescriptor& FrameInfo) override;
		virtual void OnVideoFrameReceived(TSharedPtr<IRivermaxVideoSample> InReceivedVideoFrame) override;
		virtual void OnVideoFrameReceptionError(TSharedPtr<IRivermaxVideoSample> InReceivedVideoFrame) override;
		virtual void OnStreamError() override;
		virtual void OnVideoFormatChanged(const FRivermaxInputVideoFormatChangedInfo& NewFormatInfo) override;
		//~ End IRivermaxInputStreamListener interface

	protected:

		//~ Begin FMediaIOCorePlayerBase interface
		virtual bool IsHardwareReady() const override;
		virtual void SetupSampleChannels() override;
		virtual TSharedPtr<FMediaIOCoreTextureSampleConverter> CreateTextureSampleConverter() const override;

		virtual TSharedPtr<FMediaIOCoreTextureSampleBase> AcquireTextureSample_AnyThread() const override;
		
#if WITH_EDITOR
		virtual FString GetAnalyticsEventPrefix() const override;
		virtual void GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const override;
#endif
		//~ End FMediaIOCorePlayerBase interface

	private:

		/**
		 * Listener forwarded to the ANC input stream.
		 * Keeps ANC initialization separate from the video stream state machine.
		 */
		class FAncStreamListener : public IRivermaxInputStreamListener
		{
		public:
			explicit FAncStreamListener(FRivermaxMediaPlayer& InPlayer) : Player(InPlayer) {}
			virtual ~FAncStreamListener() = default;

			virtual void OnInitializationCompleted(const FRivermaxInputInitializationResult& Result) override {}
			virtual TSharedPtr<IRivermaxVideoSample> OnVideoFrameRequested(const FRivermaxInputVideoFrameDescriptor& FrameInfo) override { return nullptr; }
			virtual void OnVideoFrameReceived(TSharedPtr<IRivermaxVideoSample> InReceivedVideoFrame) override {}
			virtual void OnStreamError() override {}
			virtual void OnVideoFormatChanged(const FRivermaxInputVideoFormatChangedInfo& NewFormatInfo) override {}

			virtual TSharedPtr<IRivermaxAncSample> OnAncFrameRequested(const FRivermaxInputAncFrameDescriptor& FrameInfo) override
			{
				return Player.HandleAncFrameRequested(FrameInfo);
			}
			virtual void OnAncFrameReceived(TSharedPtr<IRivermaxAncSample> InReceivedAncFrame) override
			{
				Player.HandleAncFrameReceived(InReceivedAncFrame);
			}
			virtual void OnAncFrameReceptionError() override
			{
				Player.HandleAncFrameReceptionError();
			}

		private:
			FRivermaxMediaPlayer& Player;
		};

		/** Called by FAncStreamListener when an ANC packet arrives. */
		TSharedPtr<IRivermaxAncSample> HandleAncFrameRequested(const FRivermaxInputAncFrameDescriptor& FrameInfo);
		void HandleAncFrameReceived(TSharedPtr<IRivermaxAncSample> InReceivedAncFrame);
		void HandleAncFrameReceptionError();

		/** Sets up stream options based on the settings of the media source */
		bool ConfigureStream(const IMediaOptions* Options);

		/** Allocates the sample pool used to receive incoming data */
		void AllocateBuffers(const FIntPoint& InResolution);

		/** Buffer upload setup that will wait on its own task to wait for sample and do the upload */
		void SampleUploadSetupTaskThreadMode(TSharedPtr<FRivermaxMediaTextureSample> Sample, FSampleConverterOperationSetup& OutConverterSetup);

		/** Function waiting for the expected frame to be received. */
		using FWaitConditionFunc = TUniqueFunction<bool(const TSharedPtr<FRivermaxMediaTextureSample>&)>;
		bool WaitForSample(const TSharedPtr<FRivermaxMediaTextureSample>& Sample, const uint64 AwaitingFrameNumber, FWaitConditionFunc WaitConditionFunction, const double TimeoutSeconds);
		
		/** Called after sample was converted / rendered. Used write a fence to detect when sample is reusable */
		void PostSampleUsage(FRDGBuilder& GraphBuilder, TSharedPtr<FRivermaxMediaTextureSample> Sample);
		
		/** Whether player is ready to play */
		bool IsReadyToPlay() const;

		/** Waits for tasks in flight and flushes render commands before cleaning our ressources */
		void WaitForPendingTasks();

		/** Used to create texture for color encoding conversion. */
		TRefCountPtr<FRHITexture> CreateIntermediateRenderTarget(FRHICommandListImmediate& RHICmdList, const FIntPoint& InDim, EPixelFormat InPixelFormat, bool bInSRGB);

	private:
		/** Size of the sample pool. The max number of FrameDelay (4) + 2 frames (to give time to return back to the pool) */
		static constexpr uint32 kMaxNumVideoFrameBuffer = 6;

		/** Current state of the media player. */
		EMediaState RivermaxThreadNewState;

		/** Options used to configure the stream. i.e.  */
		FRivermaxInputStreamOptions StreamOptions;

		/** Maps to the current input Device */
		TUniquePtr<IRivermaxInputStream> InputStream;

		/** ANC (2110-40) input streams (one per configured ANC stream). */
		TArray<TUniquePtr<IRivermaxInputStream>> AncInputStreams;

		/** Listeners forwarded to AncInputStreams to keep ANC callbacks separate from video state. */
		TArray<TUniquePtr<FAncStreamListener>> AncStreamListeners;

		/** Decoded ANC payload for one video frame. Add a TOptional field per new ANC type. */
		struct FVideoFrameAncData
		{
			// ATC LTC (see AncTimecodeDID/AncTimecodeSDID)
			TOptional<FTimecode> Timecode;
		};

		/** Keyed by RTP timestamp (shared 90 kHz PTP clock between ST2110-20 and -40). */
		TMap<uint32, FVideoFrameAncData> PendingAncData;
		FCriticalSection PendingAncDataLock;

		/** Pixel format provided by media source */
		ERivermaxPixelFormat DesiredPixelFormat = ERivermaxPixelFormat::RGB_10bit;

		/** 
		* Pool of samples. The pool has full management of shared pointers.
		* Unreferenced shared pointers automatically return back to the pool and released only when pool is destroyed. 
		*/
		TUniquePtr<FRivermaxMediaTextureSamplePool> VideoTextureSamplePool;

		/** Sample that input stream should write to in framelocking mode. */
		TStaticArray<TSharedPtr<FRivermaxMediaTextureSample>, kMaxNumVideoFrameBuffer> FrameLockedSamples;

		/** Number of tasks currently in progress. Used during shutdown to know when we are good to continue */
		std::atomic<uint32> TasksInFlight = 0;

		/** Whether the created stream supports GPUDirect. Will be confirmed after initialization. */
		bool bStreamSupportsGPUDirect = false;

		/** Time to sleep when waiting for an operation to complete */
		static constexpr double SleepTimeSeconds = 50.0 * 1E-6;

		/** Critical section used when accessing stream resolution and detect pending changes */
		mutable FCriticalSection StreamResolutionCriticalSection;

		/** Resolution detected by our stream. */
		FIntPoint StreamResolution = FIntPoint::ZeroValue;
		
		/** Whether the player follows resolution detected by our stream, adjusting texture size as required */
		bool bFollowsStreamResolution = true;

		/** Used to make sure that the player didn't accidentally skipped the reception of any frames. */
		uint32 LastFrameToAttemptReception = 0;

		/** Critical section used when accessing ProxySampleDummy. */
		mutable FCriticalSection ProxySampleAccessCriticalSection;

		/** This is the proxy sample contains all common settings for texture samples for this player. */
		TSharedPtr<FRivermaxMediaTextureSample> ProxySampleDummy;
	};
}


