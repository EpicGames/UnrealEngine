// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxMediaSource.h"

#include "MediaIOCorePlayerBase.h"
#include "RivermaxMediaSourceOptions.h"
#include "RivermaxMediaUtils.h"


/*
 * IMediaOptions interface
 */

URivermaxMediaSource::URivermaxMediaSource() : UCaptureCardMediaSource()
{
	Deinterlacer = nullptr;
	bRenderJIT = true;
	EvaluationType = EMediaIOSampleEvaluationType::Latest;
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bOverrideSourceEncoding_DEPRECATED = false;
	bOverrideSourceColorSpace_DEPRECATED = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

bool URivermaxMediaSource::GetMediaOption(const FName& Key, bool DefaultValue) const
{
	if (Key == RivermaxMediaOption::UseGPUDirect)
	{
		return VideoStream.bUseGPUDirect;
	}
	if (Key == RivermaxMediaOption::OverrideResolution)
	{
		return VideoStream.bOverrideResolution;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

int64 URivermaxMediaSource::GetMediaOption(const FName& Key, int64 DefaultValue) const
{
	if (Key == RivermaxMediaOption::Port)
	{
		return VideoStream.Port;
	}
	else if (Key == RivermaxMediaOption::PixelFormat)
	{
		return (int64)VideoStream.PixelFormat;
	}
	else if (Key == RivermaxMediaOption::AncStreamType)
	{
		return AncStreams.IsEmpty() ? (int64)ERivermaxAncStreamType::None : (int64)AncStreams[0].StreamType;
	}
	else if (Key == RivermaxMediaOption::AncPort)
	{
		return AncStreams.IsEmpty() ? DefaultValue : AncStreams[0].Port;
	}
	else if (Key == FMediaIOCoreMediaOption::FrameRateNumerator)
	{
		return VideoStream.FrameRate.Numerator;
	}
	else if (Key == FMediaIOCoreMediaOption::FrameRateDenominator)
	{
		return VideoStream.FrameRate.Denominator;
	}
	else if (Key == FMediaIOCoreMediaOption::ResolutionWidth)
	{
		return VideoStream.Resolution.X;
	}
	else if (Key == FMediaIOCoreMediaOption::ResolutionHeight)
	{
		return VideoStream.Resolution.Y;
	}
	return Super::GetMediaOption(Key, DefaultValue);
}

FString URivermaxMediaSource::GetMediaOption(const FName& Key, const FString& DefaultValue) const
{
	if (Key == FMediaIOCoreMediaOption::VideoModeName)
	{
		return FString::Printf(TEXT("FormatDescriptorTodo"));
	}
	else if (Key == RivermaxMediaOption::InterfaceAddress)
	{
		return VideoStream.InterfaceAddress;
	}
	else if (Key == RivermaxMediaOption::StreamAddress)
	{
		return VideoStream.StreamAddress;
	}
	else if (Key == RivermaxMediaOption::AncInterfaceAddress)
	{
		return AncStreams.IsEmpty() ? DefaultValue : AncStreams[0].InterfaceAddress;
	}
	else if (Key == RivermaxMediaOption::AncStreamAddress)
	{
		return AncStreams.IsEmpty() ? DefaultValue : AncStreams[0].StreamAddress;
	}
	return Super::GetMediaOption(Key, DefaultValue);
}

bool URivermaxMediaSource::HasMediaOption(const FName& Key) const
{
	if ((Key == RivermaxMediaOption::InterfaceAddress) ||
		(Key == RivermaxMediaOption::StreamAddress) ||
		(Key == RivermaxMediaOption::Port) ||
		(Key == RivermaxMediaOption::PixelFormat) ||
		(Key == RivermaxMediaOption::UseGPUDirect) ||
		(Key == RivermaxMediaOption::AncStreamType) ||
		(Key == RivermaxMediaOption::AncInterfaceAddress) ||
		(Key == RivermaxMediaOption::AncStreamAddress) ||
		(Key == RivermaxMediaOption::AncPort) ||
		(Key == FMediaIOCoreMediaOption::FrameRateNumerator) ||
		(Key == FMediaIOCoreMediaOption::FrameRateDenominator) ||
		(Key == FMediaIOCoreMediaOption::ResolutionWidth) ||
		(Key == FMediaIOCoreMediaOption::ResolutionHeight) ||
		(Key == FMediaIOCoreMediaOption::VideoModeName))
	{
		return true;
	}

	return Super::HasMediaOption(Key);
}

void URivermaxMediaSource::Serialize(FArchive& Ar_Asset)
{
	Super::Serialize(Ar_Asset);

	Ar_Asset.UsingCustomVersion(FRivermaxMediaVersion::GUID);
}

void URivermaxMediaSource::PostLoad()
{
	Super::PostLoad();
	// We can only recover data during editor. Proprties will be fixed during cook.
#if WITH_EDITORONLY_DATA
	const int32 RivermaxMediaVersion = GetLinkerCustomVersion(FRivermaxMediaVersion::GUID);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;

	// Flat video properties on URivermaxMediaSource moved into FRivermaxVideoStream VideoStream struct.
	if (RivermaxMediaVersion < FRivermaxMediaVersion::VideoStreamStruct)
	{
		VideoStream.bOverrideResolution = bOverrideResolution;
		VideoStream.Resolution         = Resolution;
		VideoStream.FrameRate          = FrameRate;
		VideoStream.PixelFormat        = PixelFormat;
		VideoStream.InterfaceAddress   = InterfaceAddress;
		VideoStream.StreamAddress      = StreamAddress;
		VideoStream.Port               = Port;
		VideoStream.bUseGPUDirect      = bUseGPUDirect;
		Modify();
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
#endif

}

/*
 * UMediaSource interface
 */

FString URivermaxMediaSource::GetUrl() const
{
	return TEXT("rmax://");//todo support proper url
}

bool URivermaxMediaSource::Validate() const
{
	return true;
}

#if WITH_EDITOR
FString URivermaxMediaSource::GetDescriptionString() const
{
	return FString::Format(TEXT("{0}x{1} {2} fps {3}"), {
		VideoStream.Resolution.X,
		VideoStream.Resolution.Y,
		FString::SanitizeFloat(VideoStream.FrameRate.AsDecimal()),
		StaticEnum<ERivermaxPixelFormat>()->GetDisplayNameTextByValue(static_cast<int64>(VideoStream.PixelFormat)).ToString() });
}

void URivermaxMediaSource::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
#define LOCTEXT_NAMESPACE "RivermaxMediaSource"
	const FNumberFormattingOptions NumberFormat = FNumberFormattingOptions()
		.DefaultNoGrouping();
	
	const FText ConfigHeader = LOCTEXT("MediaConfigurationHeader", "Media Configuration");

	const FText ResolutionText = FText::Format(LOCTEXT("ResolutionFormat", "{0}x{1}"), FText::AsNumber(VideoStream.Resolution.X, &NumberFormat), FText::AsNumber(VideoStream.Resolution.Y, &NumberFormat));
	const FText FramerateText = FText::Format(LOCTEXT("FramerateFormat", "{0} fps"), FText::AsNumber(VideoStream.FrameRate.AsDecimal()));
	const FText PixelFormatText = StaticEnum<ERivermaxPixelFormat>()->GetDisplayNameTextByValue(static_cast<int64>(VideoStream.PixelFormat));

	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("ResolutionLabel", "Resolution"), ResolutionText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("FramerateLabel", "Framerate"), FramerateText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("PixelFormatLabel", "Pixel Format"), PixelFormatText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("StreamAddressLabel", "Stream Address"), FText::FromString(VideoStream.StreamAddress)));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("PortLabel", "Port"), FText::AsNumber(VideoStream.Port, &NumberFormat)));

	const FText VideoSettingsHeader = LOCTEXT("VideoSettingsHeader", "Video Settings");
	const FText EnabledText = LOCTEXT("Enabled", "Enabled");
	const FText DisabledText = LOCTEXT("Disabled", "Disabled");

	OutInfoElements.Add(FInfoElement(VideoSettingsHeader, LOCTEXT("GPUDirectLabel", "GPUDirect"), VideoStream.bUseGPUDirect ? EnabledText : DisabledText));
	OutInfoElements.Add(FInfoElement(VideoSettingsHeader, LOCTEXT("ColorConversionLabel", "Color Conversion"), FText::FromString(ColorConversionSettings.ToString())));
	
#undef LOCTEXT_NAMESPACE
}
#endif //WITH_EDITOR
