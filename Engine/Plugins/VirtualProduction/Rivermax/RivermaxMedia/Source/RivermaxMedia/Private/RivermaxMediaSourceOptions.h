// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


namespace RivermaxMediaOption
{
	static const FName InterfaceAddress("InterfaceAddress");
	static const FName StreamAddress("StreamAddress");
	static const FName Port("Port");
	static const FName PixelFormat("PixelFormat");
	static const FName UseGPUDirect("UseGPUDirect");
	static const FName OverrideResolution("OverrideResolution");

	// ANC input stream (first entry in AncStreams array)
	static const FName AncStreamType("AncStreamType");
	static const FName AncInterfaceAddress("AncInterfaceAddress");
	static const FName AncStreamAddress("AncStreamAddress");
	static const FName AncPort("AncPort");
}
