// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxMediaTextureSample.h"

#include "RenderGraphUtils.h"
#include "RivermaxMediaPlayer.h"
#include "RivermaxMediaTextureSampleConverter.h"
#include "RivermaxMediaUtils.h"
#include "Templates/RefCounting.h"

namespace UE::RivermaxMedia
{
FRivermaxMediaTextureSample::FRivermaxMediaTextureSample()
	: FMediaIOCoreTextureSampleBase()
	, SampleReceivedEvent(FPlatformProcess::GetSynchEventFromPool(true))
{
}

FRivermaxMediaTextureSample::~FRivermaxMediaTextureSample()
{
	FGenericPlatformProcess::ReturnSynchEventToPool(SampleReceivedEvent);

	if (GPUBuffer.IsValid() && LockedMemory.load(std::memory_order_acquire) != nullptr)
	{
		TRefCountPtr<FRDGPooledBuffer> BufferRef = GPUBuffer;
		ENQUEUE_RENDER_COMMAND(FRivermaxUnlockBuffer)(
			[BufferRef](FRHICommandListImmediate& RHICmdList)
			{
				RHICmdList.UnlockBuffer(BufferRef->GetRHI());
			});
		FlushRenderingCommands();
	}
}
const FMatrix& FRivermaxMediaTextureSample::GetYUVToRGBMatrix() const
{
	return MediaShaders::YuvToRgbRec709Scaled;
}

IMediaTextureSampleConverter* FRivermaxMediaTextureSample::GetMediaTextureSampleConverter()
{
	return Converter.Get();
}

bool FRivermaxMediaTextureSample::InitializeJITR(const FMediaIOCoreSampleJITRConfigurationArgs& Args)
{
	if (!FMediaIOCoreTextureSampleBase::InitializeJITR(Args))
	{
		return false;
	}

	ERivermaxPixelFormat DesiredPixelFormat = StaticCastSharedPtr<FRivermaxMediaPlayer>(Args.Player)->GetDesiredPixelFormat();
	SetInputFormat(DesiredPixelFormat);

	return true;
}

void FRivermaxMediaTextureSample::CopyConfiguration(const TSharedPtr<FMediaIOCoreTextureSampleBase>& SourceSample)
{
	FMediaIOCoreTextureSampleBase::CopyConfiguration(SourceSample);
	TSharedPtr<FRivermaxMediaTextureSample> RivermaxSample = StaticCastSharedPtr<FRivermaxMediaTextureSample>(SourceSample);
	SetInputFormat(RivermaxSample->InputFormat);
}

void FRivermaxMediaTextureSample::InitializeGPUBuffer(const FIntPoint& InResolution, ERivermaxPixelFormat InSampleFormat, bool bSupportsGPUDirect)
{
	check(LockedMemory == nullptr);
	using namespace UE::RivermaxMediaUtils::Private;
	const FSourceBufferDesc BufferDescription = GetBufferDescription(InResolution, InSampleFormat);

	FRDGBufferDesc RDGBufferDesc = FRDGBufferDesc::CreateStructuredDesc(BufferDescription.BytesPerElement, BufferDescription.NumberOfElements);

	if (bSupportsGPUDirect)
	{
		// Required to share resource across different graphics API (DX, Cuda) for GPU-direct RDMA writes
		RDGBufferDesc.Usage |= EBufferUsageFlags::Shared;
	}
	else
	{
		// Non-GPU-direct: allocate in host-visible memory (HOST_VISIBLE|HOST_COHERENT on Vulkan, upload heap on D3D12)
		// so both CPU (ProcessSRD via LockBuffer PersistentMapping) and GPU (shader via RegisterExternalBuffer) can access it
		// without any staging copies or vkCmdCopyBuffer passes.
		RDGBufferDesc.Usage |= EBufferUsageFlags::Dynamic;
	}

	InputFormat = InSampleFormat;
	ENQUEUE_RENDER_COMMAND(FRivermaxMediaTextureSample)(
	[SharedThis = StaticCastSharedRef<FRivermaxMediaTextureSample>(AsShared()), RDGBufferDesc, bSupportsGPUDirect](FRHICommandListImmediate& CommandList)
	{
		SharedThis->GPUBuffer = AllocatePooledBuffer(RDGBufferDesc, TEXT("RmaxInput Buffer"));

		if (!bSupportsGPUDirect)
		{
			// Permanently lock the Dynamic (host-visible) buffer once at allocation time.
			// The buffer is intentionally never unlocked. ProcessSRD will write directly to LockedMemory on the Rivermax receive thread, eliminating the intermediate CPU copy.
			const uint32 BufferSize = SharedThis->GPUBuffer->GetSize();
			SharedThis->LockedMemory = CommandList.LockBuffer(SharedThis->GPUBuffer->GetRHI(), 0, BufferSize, EResourceLockMode::RLM_WriteOnly_NoOverwrite);
		}
	});
}

bool FRivermaxMediaTextureSample::IsCacheable() const
{
	return false;
}

uint8* FRivermaxMediaTextureSample::GetVideoBufferRawPtr(uint32 VideoBufferSize)
{
	// For non-GPU-direct Dynamic buffers, LockedMemory is permanently mapped at buffer init.
	void* Ptr = LockedMemory.load(std::memory_order_acquire);
	if (Ptr)
	{
		return reinterpret_cast<uint8*>(Ptr);
	}
	return nullptr;
}

void FRivermaxMediaTextureSample::SetReceptionState(ESampleState NewState)
{
	if (UE_LOG_ACTIVE(LogRivermaxMedia, VeryVerbose))
	{
		const TCHAR*& PreviousStateString = SampleStateToStringRef(GetReceptionState());
		const TCHAR*& NewStateString = SampleStateToStringRef(NewState);
		UE_LOGF(LogRivermaxMedia, VeryVerbose, "Changing state for frame number: %llu , Preivous State: %ls, New state: %ls", GetFrameNumber(), PreviousStateString, NewStateString);
	}
	
	UE::RivermaxCore::IRivermaxVideoSample::SetReceptionState(NewState);
}

ERivermaxPixelFormat FRivermaxMediaTextureSample::GetInputFormat() const
{
	return InputFormat;
}

void FRivermaxMediaTextureSample::SetInputFormat(ERivermaxPixelFormat InFormat)
{
	InputFormat = InFormat;

	switch (InputFormat)
	{
		case ERivermaxPixelFormat::RGB_12bit:
			// Falls through
		case ERivermaxPixelFormat::RGB_16bit_Float:
		{
			SampleFormat = EMediaTextureSampleFormat::FloatRGBA;
			break;
		}
		case ERivermaxPixelFormat::RGB_10bit:
			// Falls through
		case ERivermaxPixelFormat::YUV422_10bit:
		{
			SampleFormat = EMediaTextureSampleFormat::CharBGR10A2;
			break;
		}
		case ERivermaxPixelFormat::YUV422_8bit:
			// Falls through
		case ERivermaxPixelFormat::RGB_8bit:
			// Falls through
		default:
		{
			SampleFormat = EMediaTextureSampleFormat::CharBGRA;
			break;
		}
	}
}

TRefCountPtr<FRDGPooledBuffer> FRivermaxMediaTextureSample::GetGPUBuffer() const
{
	return GPUBuffer;
}


}

