// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaTextureSample.h"

#include "IRivermaxInputStream.h"
#include "MediaIOCoreTextureSampleBase.h"
#include "MediaIOCoreSamples.h"
#include "MediaShaders.h"
#include "RivermaxMediaLog.h"
#include "RivermaxMediaSource.h"
#include "Templates/SharedPointer.h"

class FEvent;

namespace UE::RivermaxMedia
{
	/**
	 * Implements a media texture sample for RivermaxMediaPlayer.  
	 */
	class FRivermaxMediaTextureSample : public FMediaIOCoreTextureSampleBase, public UE::RivermaxCore::IRivermaxVideoSample
	{
	public:
		FRivermaxMediaTextureSample();
		virtual ~FRivermaxMediaTextureSample() override;

		/** A helper function that returns the string based on the provided state. */
		static inline const TCHAR*& SampleStateToStringRef(ESampleState InState)
		{
			static const TCHAR* States[] = {
				TEXT("Idle"), TEXT("ReadyForReception"), TEXT("Received"), TEXT("ReceptionError"), TEXT("Rendering")
			};

			return States[(uint8)InState];
		}

		//~ Begin IMediaTextureSample interface
		virtual bool IsCacheable() const override;
		virtual const FMatrix& GetYUVToRGBMatrix() const override;

	#if WITH_ENGINE
		virtual IMediaTextureSampleConverter* GetMediaTextureSampleConverter() override;
	#endif
		//~ End IMediaTextureSample interface

		//~ Begin FMediaIOCoreTextureSampleBase interface
		virtual bool InitializeJITR(const FMediaIOCoreSampleJITRConfigurationArgs& Args) override;
		virtual void CopyConfiguration(const TSharedPtr<FMediaIOCoreTextureSampleBase>& SourceSample) override;
		//~ End FMediaIOCoreTextureSampleBase interface

		/** Initialized a RDG buffer based on the description required. Only useful for gpudirect functionality */
		void InitializeGPUBuffer(const FIntPoint& InResolution, ERivermaxPixelFormat InSampleFormat, bool bSupportsGPUDirect);

		/** Returns the incoming sample format */
		ERivermaxPixelFormat GetInputFormat() const;
		void SetInputFormat(ERivermaxPixelFormat InFormat);

		// Inherited via IRivermaxVideoSample
		virtual TRefCountPtr<FRDGPooledBuffer> GetGPUBuffer() const override;
		virtual uint8* GetVideoBufferRawPtr(uint32 VideoBufferSize) override;
		virtual void SetReceptionState(ESampleState State) override;

		virtual void InitializePoolable() override {};

		virtual void ShutdownPoolable() override 
		{
			// When ShutdownPoolable is called, it means that that the sample is returned back to the pool or it is done being rendered.
			// We should never be in the state where this awaits GPU transfer while being released.
			checkf(IsAwaitingForGPUTransfer() == false, TEXT("This Sample is still transferring data to GPU. If this hits, something went wrong."));
			// This means that the sample is being held and managed by the Player
			FScopeLock Lock(&StateChangeCriticalSecion);

			// This sample is done and anything that waits for this sample from now on should wait until it is received again.
			GetSampleReceivedEvent()->Reset();

			// When this sample is returned back to the pool, it means that it is done rendering and is released from sample container.
			SetReceptionState(IRivermaxSample::ESampleState::Idle);
			if (SampleConversionFence.IsValid())
			{
				SampleConversionFence->Clear();
			}

			MarkRenderingComplete();
		};

		virtual bool IsReadyForReuse() override
		{
			return !IsBeingRendered();
		}

		/** 
		* Attempts to lock this sample for rendering. 
		* Returns true if sample is ok to be rendered. 
		* Returns false if sample is already being rendered. 
		*/
		bool TryLockForRendering()
		{
			FScopeLock Lock(&StateChangeCriticalSecion);
			if (bIsPendingRendering)
			{
				// sample is already being rendered.
				return false;
			}
			else
			{
				bIsPendingRendering = true;
				return true;
			}
		};

		/** 
		* Marks that this sample can be rendered again if need be.
		*/
		void MarkRenderingComplete()
		{
			FScopeLock Lock(&StateChangeCriticalSecion);
			bIsPendingRendering = false;
		};

		bool IsBeingRendered()
		{
			FScopeLock Lock(&StateChangeCriticalSecion);
			return bIsPendingRendering;
		};

		FEvent* GetSampleReceivedEvent()
		{
			return SampleReceivedEvent;
		}
		
		void SetTimecode(const TOptional<FTimecode>& InTimecode)
		{
			Timecode = InTimecode;
		}

	public:
		/** 
		 *  Locked memory of gpu buffer when uploading. Set only for non-GPU-direct Dynamic buffers: locked once at InitializeGPUBuffer and unlocked in the
		 *  destructor. ProcessSRD writes directly here via GetVideoBufferRawPtr, eliminating any intermediate CPU copy. Null for GPU-direct buffers. 
		*/
		std::atomic<void*> LockedMemory = nullptr;

		/** Write fence enqueued after sample conversion to know when it's ready to be reused */
		FGPUFenceRHIRef SampleConversionFence;

	private:
		/** This event isued to wait for the sample to be fully received.*/
		FEvent* SampleReceivedEvent = nullptr;

		/** True when queued for rendering. Will be false once fence has been written, after shader usage. */
		std::atomic<bool> bIsPendingRendering = false;

	private:

		/** Format in the rivermax realm */
		ERivermaxPixelFormat InputFormat = ERivermaxPixelFormat::YUV422_8bit;

		/** Texture stride */
		uint32 Stride = 0;

		/** Pooled buffer used for gpudirect functionality. Received content will already be on GPU when received from NIC */
		TRefCountPtr<FRDGPooledBuffer> GPUBuffer;

	private:
		friend class FRivermaxMediaPlayer;

		/** RTP timestamp of this frame (90 kHz PTP clock). Used to look up the matching ANC entry. */
		uint32 RTPTimestamp = 0;

		/** The start of the reception marked by the first chunk received by rivermax. */
		FTimespan FrameReceptionStart = 0;

		/** The end of the reception marked by the last processed packet. */
		FTimespan FrameReceptionEnd = 0;
	};

	class FRivermaxMediaTextureSamplePool : public TMediaObjectPool<FRivermaxMediaTextureSample> { };
}

