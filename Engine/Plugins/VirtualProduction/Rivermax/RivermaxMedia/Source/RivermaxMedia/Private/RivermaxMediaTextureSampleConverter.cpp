// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxMediaTextureSampleConverter.h"

namespace UE::RivermaxMedia
{
	uint32 FRivermaxMediaTextureSampleConverter::GetConverterInfoFlags() const
	{
		if (TSharedPtr<FMediaIOCoreTextureSampleBase> ProxySampleSharedPtr = JITRProxySample.Pin())
		{
			if (ProxySampleSharedPtr->HasEncodingOverride() || ProxySampleSharedPtr->HasColorSpaceOverride())
			{
				return ConverterInfoFlags_WillCreateOutputTexture | ConverterInfoFlags_PreprocessOnly;
			}
		}
		return ConverterInfoFlags_WillCreateOutputTexture;
	}
}