// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaIOCoreTextureSampleConverter.h"

#include "RivermaxMediaSource.h"


class FRDGPooledBuffer;
class FRDGBuilder;

namespace UE::RivermaxMedia
{
	class FRivermaxMediaTextureSample;
	class FRivermaxMediaPlayer;

	using FPreInputConvertFunc = TUniqueFunction<void(FRDGBuilder& GraphBuilder)>;
	using FGetGPUBufferFunc = TFunction<TRefCountPtr<FRDGPooledBuffer>()>;
	using FPostInputConvertFunc = TUniqueFunction<void(FRDGBuilder& GraphBuilder)>;

	/** Structure used during late update to let player configure some operations */
	struct FSampleConverterOperationSetup
	{
		/** Function to be called before setting up the sample conversion graph */
		FPreInputConvertFunc PreConvertFunc = nullptr;

		/** Function used to retrieve gpu buffer if available */
		FGetGPUBufferFunc GetGPUBufferFunc = nullptr;

		/** Function to be called after setting up the sample conversion graph */
		FPostInputConvertFunc PostConvertFunc = nullptr;
	};

	class FRivermaxMediaTextureSampleConverter : public FMediaIOCoreTextureSampleConverter
	{
	public:
		virtual uint32 GetConverterInfoFlags() const override;
	};

}
