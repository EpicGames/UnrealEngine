// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxMediaUtils.h"

#include "RivermaxMediaLog.h"
#include "RivermaxShaders.h"
#include "Serialization/CustomVersion.h"

// Guid generated by FGuid::NewGuid
const FGuid FRivermaxMediaVersion::GUID(0x867597A5, 0x492791B6, 0x3FDACDB0, 0x60344667);

// RivermaxMedia Version registration with core.
FCustomVersionRegistration GRegisterRivermaxMediaCustomVersion(FRivermaxMediaVersion::GUID, FRivermaxMediaVersion::LatestVersion, TEXT("RivermaxMediaVersion"));


namespace UE::RivermaxMediaUtils::Private
{
	UE::RivermaxCore::ESamplingType PixelFormatToRivermaxSamplingType(ERivermaxPixelFormat InPixelFormat)
	{
		using namespace UE::RivermaxCore;

		switch (InPixelFormat)
		{
		case ERivermaxPixelFormat::YUV422_8bit:    return ESamplingType::YUV422_8bit;
		case ERivermaxPixelFormat::YUV422_10bit:   return ESamplingType::YUV422_10bit;
		case ERivermaxPixelFormat::RGB_8bit:       return ESamplingType::RGB_8bit;
		case ERivermaxPixelFormat::RGB_10bit:      return ESamplingType::RGB_10bit;
		case ERivermaxPixelFormat::RGB_12bit:      return ESamplingType::RGB_12bit;
		case ERivermaxPixelFormat::RGB_16bit_Float: return ESamplingType::RGB_16bitFloat;
		default:
			checkNoEntry();
			return ESamplingType::RGB_10bit;
		}
	}

	FSourceBufferDesc GetBufferDescription(const FIntPoint& InResolution, ERivermaxPixelFormat InPixelFormat)
	{
		using namespace UE::RivermaxCore;
		using namespace UE::RivermaxShaders;

		FSourceBufferDesc Description;

		const ESamplingType SamplingType = PixelFormatToRivermaxSamplingType(InPixelFormat);
		const FVideoFormatInfo Info = FStandardVideoFormat::GetVideoFormatInfo(SamplingType);

		// Compute horizontal byte count (stride) of resolution
		const uint32 PixelAlignment = Info.PixelGroupCoverage;
		const uint32 PixelCount = InResolution.X * InResolution.Y;
		ensureMsgf(PixelCount % PixelAlignment == 0, TEXT("Unaligned resolution (%dx%d) provided for pixel format %s"), InResolution.X, InResolution.Y, *UEnum::GetValueAsString(InPixelFormat));
		const uint32 PixelGroupCount = PixelCount / PixelAlignment;
		const uint32 FrameByteCount = PixelGroupCount * Info.PixelGroupSize;

		switch (SamplingType)
		{
		case ESamplingType::YUV422_8bit:
		{
			Description.BytesPerElement = sizeof(FRGBToYUV8Bit422CS::FYUV8Bit422Buffer);
			break;
		}
		case ESamplingType::YUV422_10bit:
		{
			Description.BytesPerElement = sizeof(FRGBToYUV10Bit422LittleEndianCS::FYUV10Bit422LEBuffer);
			break;
		}
		case ESamplingType::RGB_8bit:
		{
			Description.BytesPerElement = sizeof(FRGBToRGB8BitCS::FRGB8BitBuffer);
			break;
		}
		case ESamplingType::RGB_10bit:
		{
			Description.BytesPerElement = sizeof(FRGB10BitToRGBA10CS::FRGB10BitBuffer);
			break;
		}
		case ESamplingType::RGB_12bit:
		{
			Description.BytesPerElement = sizeof(FRGBToRGB12BitCS::FRGB12BitBuffer);
			break;
		}
		case ESamplingType::RGB_16bitFloat:
		{
			Description.BytesPerElement = sizeof(FRGBToRGB16fCS::FRGB16fBuffer);
			break;
		}
		default:
		{
			checkNoEntry();
			return Description;
		}
		}

		// Shader encoding might not align with pixel group size so we need to have enough elements to represent the last pixel group
		Description.NumberOfElements = FrameByteCount / Description.BytesPerElement;
		Description.NumberOfElements += FrameByteCount % Description.BytesPerElement != 0 ? 1 : 0;

		return Description;
	}

	FIntPoint GetAlignedResolution(const UE::RivermaxCore::FVideoFormatInfo& InFormatInfo, const FIntPoint& ResolutionToAlign)
	{
		// We need to adjust horizontal resolution if it's not aligned to pgroup pixel coverage
		// RTP header has to describe a full pgroup even if it's outside of effective resolution. Refer to st2110-20: 6.2.2 pgroup Size and Construction  
		const uint32 PixelAlignment = InFormatInfo.PixelGroupCoverage;
		uint32 PixelsToBeFitIntoGroup = ResolutionToAlign.X % PixelAlignment;
		const uint32 AlignedHorizontalResolution = PixelsToBeFitIntoGroup > 0 ? ResolutionToAlign.X + (PixelAlignment - PixelsToBeFitIntoGroup) : ResolutionToAlign.X;

		// This is a workaround for an issue with Multi-SRD packets. We pad to the packet with the smallest requirement for padding 
		// so that divides the pixel row into equal sizes. To be removed when Multi-SRD is fixed.
		uint32 AlignedHorizontalResolutionPacketSRD = AlignedHorizontalResolution;

		IConsoleVariable* CvarEnableMultiSrd = IConsoleManager::Get().FindConsoleVariable(TEXT("Rivermax.Output.EnableMultiSRD"));

		if (CvarEnableMultiSrd && CvarEnableMultiSrd->GetInt() == 0)
		{
			// PixelGroupSize is always exactly divisible by PixelGroupCoverage;
			const uint32 BytesPerRow = (AlignedHorizontalResolution  * InFormatInfo.PixelGroupSize) / InFormatInfo.PixelGroupCoverage;

			FPayloadSizeInformation PayloadSizeInfo;
			FindPayloadSize(BytesPerRow, InFormatInfo.PixelGroupSize, PayloadSizeInfo);

			// Pad if the appropriate payload size isn't found.
			if (!PayloadSizeInfo.bIsExactFit)
			{
				const uint32 BytesPerRowAligned = ((BytesPerRow + PayloadSizeInfo.PayloadSize - 1) / PayloadSizeInfo.PayloadSize) * PayloadSizeInfo.PayloadSize;
				AlignedHorizontalResolutionPacketSRD = (BytesPerRowAligned * InFormatInfo.PixelGroupCoverage)  / InFormatInfo.PixelGroupSize;
			}
		}

		return FIntPoint(AlignedHorizontalResolutionPacketSRD, ResolutionToAlign.Y);
	}

	UE::RivermaxCore::ERivermaxAlignmentMode MediaOutputAlignmentToRivermaxAlignment(ERivermaxMediaAlignmentMode InAlignmentMode)
	{
		using namespace UE::RivermaxCore;

		switch (InAlignmentMode)
		{
		case ERivermaxMediaAlignmentMode::AlignmentPoint:
		{
			return ERivermaxAlignmentMode::AlignmentPoint;
		}
		case ERivermaxMediaAlignmentMode::FrameCreation:
		{
			return ERivermaxAlignmentMode::FrameCreation;
		}
		default:
		{
			checkNoEntry();
			return ERivermaxAlignmentMode::AlignmentPoint;
		}
		}
	}

	ERivermaxPixelFormat RivermaxSamplingTypeToPixelFormat(UE::RivermaxCore::ESamplingType InSamplingType)
	{
		using namespace UE::RivermaxCore;

		switch (InSamplingType)
		{
		case ESamplingType::YUV422_8bit:   return ERivermaxPixelFormat::YUV422_8bit;
		case ESamplingType::YUV422_10bit:  return ERivermaxPixelFormat::YUV422_10bit;
		case ESamplingType::RGB_8bit:      return ERivermaxPixelFormat::RGB_8bit;
		case ESamplingType::RGB_10bit:     return ERivermaxPixelFormat::RGB_10bit;
		case ESamplingType::RGB_12bit:     return ERivermaxPixelFormat::RGB_12bit;
		case ESamplingType::RGB_16bitFloat: return ERivermaxPixelFormat::RGB_16bit_Float;
		default:
			checkNoEntry();
			return ERivermaxPixelFormat::RGB_10bit;
		}
	}

	FString PixelFormatToSamplingDesc(UE::RivermaxCore::ESamplingType SamplingType)
	{
		using namespace UE::RivermaxCore;
		switch (SamplingType)
		{
		case ESamplingType::RGB_8bit:
		case ESamplingType::RGB_10bit:
		case ESamplingType::RGB_12bit:
		case ESamplingType::RGB_16bit:
		case ESamplingType::RGB_16bitFloat:
		{
			return FString(TEXT("RGB"));
		}
		case ESamplingType::YUV422_8bit:
		case ESamplingType::YUV422_10bit:
		case ESamplingType::YUV422_12bit:
		case ESamplingType::YUV422_16bit:
		case ESamplingType::YUV422_16bitFloat:
		{
			return FString(TEXT("YCbCr-4:2:2"));
		}
		case ESamplingType::YUV444_8bit:
		case ESamplingType::YUV444_10bit:
		case ESamplingType::YUV444_12bit:
		case ESamplingType::YUV444_16bit:
		case ESamplingType::YUV444_16bitFloat:
		{
			return FString(TEXT("YCbCr-4:4:4"));
		}
		default:
		{
			checkNoEntry();
			return FString();
		}
		}
	}

	FString PixelFormatToBitDepth(UE::RivermaxCore::ESamplingType SamplingType)
	{
		using namespace UE::RivermaxCore;
		switch (SamplingType)
		{
		case ESamplingType::RGB_8bit:
		case ESamplingType::YUV422_8bit:
		case ESamplingType::YUV444_8bit:
		{
			return FString(TEXT("8"));
		}
		case ESamplingType::RGB_10bit:
		case ESamplingType::YUV422_10bit:
		case ESamplingType::YUV444_10bit:
		{
			return FString(TEXT("10"));
		}
		case ESamplingType::RGB_12bit:
		case ESamplingType::YUV422_12bit:
		case ESamplingType::YUV444_12bit:
		{
			return FString(TEXT("12"));
		}
		case ESamplingType::RGB_16bit:
		case ESamplingType::YUV422_16bit:
		case ESamplingType::YUV444_16bit:
		{
			return FString(TEXT("16"));
		}
		case ESamplingType::RGB_16bitFloat:
		case ESamplingType::YUV422_16bitFloat:
		case ESamplingType::YUV444_16bitFloat:
		{
			return FString(TEXT("16f"));
		}
		default:
		{
			checkNoEntry();
			return FString();
		}
		}
	}

	FIntVector GetComputeShaderGroupCount(uint32 NumberOfElements)
	{
		constexpr uint32 GX = 512u;
		constexpr uint32 GY = 1u;
		constexpr uint32 ThreadsPerGroup = GX * GY;

		const uint32 MaxGroupsPerDim = GRHIMaxDispatchThreadGroupsPerDimension.X; // typically 65535

		const uint64 GroupsNeeded = FMath::DivideAndRoundUp((uint64)NumberOfElements, (uint64)ThreadsPerGroup);

		const uint32 GroupsX = (uint32)FMath::Min<uint64>(GroupsNeeded, MaxGroupsPerDim);

		//TODO:
		// GPU compute dispatches have maximum workgroup counts per dimension. 
		// Unreal exposes these limits via GRHIMaxDispatchThreadGroupsPerDimension.X/Y/Z. The current implementation clamps X 
		// meaning that 16 bit 8k*8k resolution gets clipped. In the future release we should add multi-dimensional dispatch support.
		return FIntVector(GroupsX, 1, 1);
	}

	bool StreamOptionsToSDPDescription(const UE::RivermaxCore::FRivermaxOutputOptions& Options, TArray<char>& OutSDPDescription)
	{
		uint8 NumberOfStreams = 0; 

		// There needs to be at least one valid stream for SDP to be created.
		for (const TSharedPtr<UE::RivermaxCore::FRivermaxOutputStreamOptions>& Stream : Options.StreamOptions)
		{
			if (Stream.IsValid())
			{
				NumberOfStreams++;
			}
		}

		if (NumberOfStreams == 0)
		{
			return false;
		}

		// Basic SDP string creation from a set of options. At some point, having a proper SDP loader / creator would be useful.
		// Refer to https://datatracker.ietf.org/doc/html/rfc4570


		constexpr int32 MulticastTTL = 32;
		TAnsiStringBuilder<2048> SDPDescriptionBuilder;

		// Protocol version.
		SDPDescriptionBuilder.Appendf("v=0\n");

		// Session name.
		SDPDescriptionBuilder.Appendf("s=Unreal Engine Rivermax 2110 \n");

		// Start and stop time of the session. 0 0 means permanently active.
		SDPDescriptionBuilder.Appendf("t=0 0\n");

		// Group streams. Currently unsupported by Rivermax.
		if (NumberOfStreams > 1)
		{
			// This isn't supported by Rivermax 1.60 and the Rivermax sdp parser will fail without giving a clear reason.
			// Keeping it here for potential future support.
			//SDPDescriptionBuilder.Appendf("a=group:FID V1 M1\n");
		}

		// This is used by RivermaxOutStream when it calls to the library to assign Media Block Index in SDP.
		uint64 StreamIndex = 0;

		using namespace UE::RivermaxCore;

		// Adding Video stream properties to the SDP
		TSharedPtr<FRivermaxVideoOutputOptions> VideoOptions 
			= StaticCastSharedPtr<FRivermaxVideoOutputOptions>(Options.StreamOptions[ERivermaxStreamType::ST2110_20]);

		if (VideoOptions.IsValid())
		{

			// Apply desired multiplier and convert fractional frame rate to be represented over 1001 for sdp compliance.
			FFrameRate DesiredRate = VideoOptions->FrameRate;
			const double DecimalDesiredRate = VideoOptions->FrameRate.AsDecimal();
			const bool bIsNonStandardFractionalFrameRate = (DesiredRate.Denominator != 1001 && !FMath::IsNearlyZero(FMath::Frac(DecimalDesiredRate)));
			if (bIsNonStandardFractionalFrameRate)
			{
				UE_LOGF(LogRivermaxMedia, Warning, "Fractional frame rates must be described using a denominator of 1001. Converting it for stream creation.");
			}

			FString FrameRateDescription;
			if (bIsNonStandardFractionalFrameRate)
			{
				const double NewRateDecimal = DecimalDesiredRate * 1001;
				DesiredRate.Numerator = FMath::RoundToInt32(NewRateDecimal);
				DesiredRate.Denominator = 1001;
			}

			if (DesiredRate.Denominator == 1001)
			{
				FrameRateDescription = FString::Printf(TEXT("%u/%u"), DesiredRate.Numerator, DesiredRate.Denominator);
			}
			else
			{
				FrameRateDescription = FString::Printf(TEXT("%u"), FMath::RoundToInt32(DesiredRate.AsDecimal()));
			}

			// Media description.
			SDPDescriptionBuilder.Appendf("m=video %d RTP/AVP 96\n", VideoOptions->Port);

			// Connection information.
			SDPDescriptionBuilder.Appendf("c=IN IP4 %S/%d\n", *VideoOptions->StreamAddress, MulticastTTL);

			// Limits multicast to a specific device.
			SDPDescriptionBuilder.Appendf("a=source-filter: incl IN IP4 %S %S\n", *VideoOptions->StreamAddress, *VideoOptions->InterfaceAddress);

			// RTP mapping attribute. Payload type and clock rate.
			SDPDescriptionBuilder.Appendf("a=rtpmap:96 raw/90000\n");

			// Video Format parameters.
			SDPDescriptionBuilder.Appendf("a=fmtp: 96 sampling=%S; width=%d; height=%d; exactframerate=%S; depth=%S; TCS=SDR; colorimetry=BT709; PM=2110GPM; SSN=ST2110-20:2017; TP=2110TPN;\n"
				, *PixelFormatToSamplingDesc(VideoOptions->PixelFormat)
				, VideoOptions->AlignedResolution.X
				, VideoOptions->AlignedResolution.Y
				, *FrameRateDescription
				, *PixelFormatToBitDepth(VideoOptions->PixelFormat));

			// sets the identifier for this stream.
			SDPDescriptionBuilder.Appendf("a=mid:V1\n");

			VideoOptions->StreamIndex = StreamIndex++;
		}


		TSharedPtr<FRivermaxAncOutputOptions> AncOptions
			= StaticCastSharedPtr<FRivermaxAncOutputOptions>(Options.StreamOptions[ERivermaxStreamType::ST2110_40_TC]);

		// Adding Anc stream properties to the SDP
		if (AncOptions.IsValid())
		{
			// Media description. 
			SDPDescriptionBuilder.Appendf("m=video %d RTP/AVP 97\n", AncOptions->Port);

			// Connection information.
			SDPDescriptionBuilder.Appendf("c=IN IP4 %S/%d\n", *AncOptions->StreamAddress, MulticastTTL);

			// Limits multicast to a specific device.
			SDPDescriptionBuilder.Appendf("a=source-filter: incl IN IP4 %S %S\n", *AncOptions->StreamAddress, *AncOptions->InterfaceAddress);

			// RTP mapping attribute. Payload type and clock rate.
			SDPDescriptionBuilder.Appendf("a=rtpmap:97 smpte291/90000\n");

			// RTP mapping attribute. Payload type and clock rate.
			SDPDescriptionBuilder.Appendf("a=fmtp:97 DID_SDID={0x%X,0x%X}\n", AncOptions->GetDIDRaw(), AncOptions->GetSDIDRaw());

			// sets the identifier for this stream.
			SDPDescriptionBuilder.Appendf("a=mid:M1\n");

			AncOptions->StreamIndex = StreamIndex++;
		}

		OutSDPDescription.SetNum(SDPDescriptionBuilder.Len());
		memcpy(&OutSDPDescription[0], SDPDescriptionBuilder.GetData(), SDPDescriptionBuilder.Len());
		OutSDPDescription.Add('\0');
		return true;
	}

	void FindPayloadSize(uint32 InBytesPerLine, const uint32 PixelGroupSize, FPayloadSizeInformation& OutPayloadSizeInformation)
	{
		OutPayloadSizeInformation.bIsExactFit = false;

		// For now, we only find a payload size that can be equal across one line
		// Support for last payload of a line being smaller is there but is causing issue
		// We fail to output if we receive a resolution for which we can't find an equal payload size
		int32 NumberOfPacketsPerLine = InBytesPerLine / MaxPayloadSize;
		if (NumberOfPacketsPerLine == 0)
		{
			if (InBytesPerLine > MinPayloadSize && InBytesPerLine % PixelGroupSize == 0)
			{
				OutPayloadSizeInformation.PayloadSize = InBytesPerLine;
				OutPayloadSizeInformation.bIsExactFit = true;
				return;
			}

			// Pad to the smallest packet size since the resolution is too small.
			OutPayloadSizeInformation.PayloadSize = FMath::Max(MinPayloadSize, InBytesPerLine);
			if (InBytesPerLine % OutPayloadSizeInformation.PayloadSize != 0)
			{
				// This may exceed MaxPayloadSize, which is acceptable since this possibility is accounted for  
				// by setting MaxPayloadSize to be a few pixel groups' worth of bytes smaller.
				OutPayloadSizeInformation.PayloadSize = OutPayloadSizeInformation.PayloadSize + PixelGroupSize - (OutPayloadSizeInformation.PayloadSize % PixelGroupSize);
			}
			OutPayloadSizeInformation.bIsExactFit = InBytesPerLine % OutPayloadSizeInformation.PayloadSize == 0;
			return;
		}

		// By default the maximum udp size aligned to the nearest pixel group size is used. Then a more appropriate payload is attempted to be found.
		uint32 ClosestPayloadSizeToPadTo = FMath::Max(MaxPayloadSize - (MaxPayloadSize % PixelGroupSize), MinPayloadSize);

		while (true)
		{
			const uint32 TestPacketSize = InBytesPerLine / NumberOfPacketsPerLine;
			if (TestPacketSize < MinPayloadSize)
			{
				break;
			}

			if (TestPacketSize <= MaxPayloadSize)
			{
				if ((TestPacketSize % PixelGroupSize) == 0)
				{
					// This means we have found a packet size that requires a smaller number of pixels to pad to.
					if (InBytesPerLine % TestPacketSize < (InBytesPerLine % ClosestPayloadSizeToPadTo))
					{
						ClosestPayloadSizeToPadTo = FMath::Max(TestPacketSize, MinPayloadSize);
					}

					if ((InBytesPerLine % NumberOfPacketsPerLine) == 0)
					{
						OutPayloadSizeInformation.PayloadSize = TestPacketSize;
						OutPayloadSizeInformation.bIsExactFit = true;
						return;
					}
				}
			}

			++NumberOfPacketsPerLine;
		}

		OutPayloadSizeInformation.PayloadSize = ClosestPayloadSizeToPadTo;
		OutPayloadSizeInformation.bIsExactFit = false;
		return;
	}

	UE::RivermaxCore::EFrameLockingMode MediaOutputFrameLockingToRivermax(ERivermaxFrameLockingMode InFrameLockingMode)
	{
		using namespace UE::RivermaxCore;

		switch (InFrameLockingMode)
		{
		case ERivermaxFrameLockingMode::FreeRun:
		{
			return EFrameLockingMode::FreeRun;
		}
		case ERivermaxFrameLockingMode::BlockOnReservation:
		{
			return EFrameLockingMode::BlockOnReservation;
		}
		default:
		{
			checkNoEntry();
			return EFrameLockingMode::FreeRun;
		}
		}
	}

}
