// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RivermaxFormats.h"
#include "RivermaxMediaOutput.h"
#include "RivermaxTypes.h"

struct FGuid;

namespace UE::RivermaxMediaUtils::Private
{
	struct FSourceBufferDesc
	{
		uint32 BytesPerElement = 0;
		uint32 NumberOfElements = 0;
	};

	struct FPayloadSizeInformation
	{
		/** True if an exact fit is found. I.e. a packet size was found that divides sample row without the remainder. */
		bool bIsExactFit = false;
		
		/**
		* If an exact fit this PayloadSize will be used without padding the resolution.
		* If not an exact fit then the PayloadSize will be used to pad the resolution to.
		*/
		uint32 PayloadSize = 0;
	};

	/** ATC LTC timecode DID/SDID per SMPTE ST 291-1. */
	static constexpr uint8 AncTimecodeDID  = 0x60;
	static constexpr uint8 AncTimecodeSDID = 0x60;

	/** Maximum payload size in bytes that the plugin can send based on UDP max size and RTP header.  */
	static constexpr uint32 MaxPayloadSize = 1420;

	/** 
	* Smallest payload size (bytes) to use as a lower bound in search for a payload that can be equal across a line.
	* The smaller the packet size, the greater the overhead. 
	* More packets mean more RTP and UDP headers need to be included, and more processing is required as a result.
	*/
	static constexpr uint32 MinPayloadSize = 600;

	UE::RivermaxCore::ESamplingType PixelFormatToRivermaxSamplingType(ERivermaxPixelFormat InPixelFormat);
	ERivermaxPixelFormat RivermaxSamplingTypeToPixelFormat(UE::RivermaxCore::ESamplingType InSamplingType);
	UE::RivermaxCore::ERivermaxAlignmentMode MediaOutputAlignmentToRivermaxAlignment(ERivermaxMediaAlignmentMode InAlignmentMode);
	UE::RivermaxCore::EFrameLockingMode MediaOutputFrameLockingToRivermax(ERivermaxFrameLockingMode InFrameLockingMode);
	FSourceBufferDesc GetBufferDescription(const FIntPoint& Resolution, ERivermaxPixelFormat InPixelFormat);

	/**
	 * Computes the dispatch group count for a 2110 buffer conversion compute shader.
	 *
	 * Each group in the shader is declared as [numthreads(512,1,1)], which means
	 * 512 threads are launched per group. In the future we should consider using multi dimensional 
	 * dispatch group count with a smaller group size.
	 * NumberOfElements - The total number of elements (e.g. pixels or items) that need to be processed.
	 * 
	 * Returns FIntVector giving the number of groups to dispatch in each dimension. 
	 * It needs to reflect DISPATCH_GROUP_COUNT in RivermaxShaders.usf.
	 */
	FIntVector GetComputeShaderGroupCount(uint32 NumberOfElements);

	/** Gets a resolution aligned to a pixel group specified for the provided video format as per 2110-20 requirements. */
	FIntPoint GetAlignedResolution(const UE::RivermaxCore::FVideoFormatInfo& InFormatInfo, const FIntPoint& ResolutionToAlign);

	/** 
	* Convert a set of streaming option to its SDP description.
	* Returns true if SDP could successfully be created from rivermax options.
	*/
	bool StreamOptionsToSDPDescription(const UE::RivermaxCore::FRivermaxOutputOptions& Options, TArray<char>& OutSDPDescription);

	/**
	* Find the right size for the data portion of the UDP packet. It is generally desired to maximize the size of the packet and make it as close
	* possible to the maximum UDP packet size which is typically 1500 with the header.
	* OutPayloadSizeInformation contains information about the packet that requires smallest padding if an exact packet that fits equal number of bytes isn't found.
	*/
	void FindPayloadSize(uint32 InBytesPerLine, const uint32 PixelGroupSize, FPayloadSizeInformation& OutPayloadSizeInformation);
}

// Custom version to keep track of and restore depricated properties.
struct RIVERMAXMEDIA_API FRivermaxMediaVersion
{
	enum Type
	{
		BeforeCustomVersionAdded = 0,

		// Flat video properties on URivermaxMediaSource moved into FRivermaxVideoStream VideoStream struct.
		VideoStreamStruct = 1,

		// ANC input stream configuration (AncStreams) added to URivermaxMediaSource.
		AncTimecodeInput = 2,

		// Add new versions above this comment.
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1


	};

	// Rivermax Guild
	const static FGuid GUID;

private:
	FRivermaxMediaVersion() {}
};