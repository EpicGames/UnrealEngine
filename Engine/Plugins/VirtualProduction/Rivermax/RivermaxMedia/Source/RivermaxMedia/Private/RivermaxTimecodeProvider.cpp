// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxTimecodeProvider.h"

#include "IRivermaxCoreModule.h"
#include "IRivermaxManager.h"
#include "RivermaxMediaLog.h"
#include "RivermaxPTPUtils.h"

#define LOCTEXT_NAMESPACE "RivermaxTimecodeProvider"


bool URivermaxTimecodeProvider::FetchTimecode(FQualifiedFrameTime& OutFrameTime)
{
	IRivermaxCoreModule* RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
	if (RivermaxModule && RivermaxModule->GetRivermaxManager() && RivermaxModule->GetRivermaxManager()->IsLibraryInitialized())
	{
		// Get rivermax clock time, and truncate to timespan tick's resolution (100ns / tick)
		const uint64 CurrentTime = RivermaxModule->GetRivermaxManager()->GetTime();
		FTimespan CurrentTimespan = FTimespan(CurrentTime / ETimespan::NanosecondsPerTick);

		// Adjust for daylight saving that might be required
		CurrentTimespan -= FTimespan(DaylightSavingTimeHourOffset, 0, 0);

		// Convert from TAI PTP Time to UTC
		CurrentTimespan -= FTimespan(0, 0, UTCSecondsOffset);

		constexpr bool bRollOver = true;
		FTimecode Timecode = FTimecode::FromTimespan(CurrentTimespan, FrameRate, bRollOver);
		OutFrameTime = FQualifiedFrameTime(Timecode, FrameRate);

		// Adjust timecode to match LTC Timecode that should be one frame late because of LTC transport. 
		OutFrameTime.Time.FrameNumber -= FFrameNumber(PTPToLTCTimecodeFrameOffset);

		return true;
	}

	return false;
}

bool URivermaxTimecodeProvider::Initialize(class UEngine* InEngine)
{
	State = ETimecodeProviderSynchronizationState::Closed;

	IRivermaxCoreModule* RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
	if(RivermaxModule == nullptr || RivermaxModule->IsAvailable() == false)
	{
		UE_LOGF(LogRivermaxMedia, Warning, "Can't initialize Rivermax TimecodeProvider, Rivermax Core Module isn't available.");
		State = ETimecodeProviderSynchronizationState::Error;
		return false;
	}

	TSharedPtr<UE::RivermaxCore::IRivermaxManager> Manager = RivermaxModule->GetRivermaxManager();
	if(!Manager->ValidateLibraryIsLoaded())
	{
		UE_LOGF(LogRivermaxMedia, Warning, "Can't initialize Rivermax TimecodeProvider, library isn't initialized.");
		State = ETimecodeProviderSynchronizationState::Error;
		return false;
	}

	if(RivermaxModule->GetRivermaxManager()->GetDevices().Num() <= 0)
	{
		UE_LOGF(LogRivermaxMedia, Warning, "Can't initialize Rivermax TimecodeProvider, no compatible devices were found.");
		State = ETimecodeProviderSynchronizationState::Error;
		return false;
	}

	const ERivermaxTimeSource TimeSource = RivermaxModule->GetRivermaxManager()->GetTimeSource();
	if ((TimeSource != ERivermaxTimeSource::PTP) && (TimeSource != ERivermaxTimeSource::System))
	{
		UE_LOGF(LogRivermaxMedia, Warning, "Can't initialize Rivermax TimecodeProvider, Rivermax clock type isn't supported.");
		State = ETimecodeProviderSynchronizationState::Error;
		return false;
	}

	if (TimeSource == ERivermaxTimeSource::System)
	{
		UE_LOGF(LogRivermaxMedia, Warning, "Rivermax clock is using system time. Make sure it's synchronized to PTP to get expected results.");
	}

	State = ETimecodeProviderSynchronizationState::Synchronized;

	return true;
}

void URivermaxTimecodeProvider::Shutdown(class UEngine* InEngine)
{
	State = ETimecodeProviderSynchronizationState::Closed;
}

#undef LOCTEXT_NAMESPACE
