// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "Modules/ModuleManager.h"

class IMediaPlayer;
class IMediaEventSink;



/**
 * 
 */
class IRivermaxMediaModule : public IModuleInterface
{
public:
	static inline IRivermaxMediaModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IRivermaxMediaModule>("RivermaxMedia");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("RivermaxMedia");
	}

	/**
	 * Create a Rivermax based media player.
	 * @param EventSink The object that receives media events from the player.
	 * @return A new media player, or nullptr if a player couldn't be created.
	 */
	virtual TSharedPtr<IMediaPlayer> CreatePlayer(IMediaEventSink& EventSink) = 0;
};

