// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenlockedCustomTimeStep.h"

#include "RivermaxCustomTimeStep.generated.h"


class UEngine;

/**
 * Genlock using PTP time from a rivermax card
 */
UCLASS(Blueprintable, editinlinenew)
class RIVERMAXMEDIA_API URivermaxCustomTimeStep : public UGenlockedCustomTimeStep
{
public:
	GENERATED_BODY()

	//~ Begin UFixedFrameRateCustomTimeStep interface
	virtual bool Initialize(UEngine* InEngine) override;
	virtual void Shutdown(UEngine* InEngine) override;
	virtual bool UpdateTimeStep(UEngine* InEngine) override;
	virtual ECustomTimeStepSynchronizationState GetSynchronizationState() const override;
	virtual FFrameRate GetFixedFrameRate() const override;
	//~ End UFixedFrameRateCustomTimeStep interface

	//~ UGenlockedCustomTimeStep interface
	virtual uint32 GetLastSyncCountDelta() const override;
	virtual bool IsLastSyncDataValid() const override;
	virtual FFrameRate GetSyncRate() const override;
	virtual bool WaitForSync() override;
	//~ End UGenlockedCustomTimeSTep interface

#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR


private:

	/** Waits for the next frame based on frame rate settings */
	bool WaitForNextFrame();

	/** We wait for Rivermax Manager to be initialized to do final verification before going to Synchronized state` */
	void OnRivermaxManagerInitialized();

	/** Adds current frame rate to boundary monitor */
	void EnableMonitoring();

	/** Removes current frame rate to boundary monitor */
	void DisableMonitoring();

public:

	/** Target frame rate to which to genlock. Uses ST2059 standard to align PTP time to standard genlock */
	UPROPERTY(EditAnywhere, Category = "Genlock options")
	FFrameRate FrameRate = FFrameRate(24,1);

	/** Delay from alignment point at which we tick the engine, in milliseconds  */
	UPROPERTY(EditAnywhere, Category = "Genlock options")
	double AlignmentPointDelayMS = 0;

	/** When enabled, will warn for skipped frames when engine is too slow */
	UPROPERTY(EditAnywhere, Category = "Genlock options", meta = (DisplayName = "Display Dropped Frames Warning"), AdvancedDisplay)
	bool bEnableOverrunDetection = true;

private:

#if WITH_EDITORONLY_DATA
	/** Engine used to initialize the Provider */
	UPROPERTY(Transient)
	TObjectPtr<UEngine> InitializedEngine = nullptr;
#endif

	/** The current SynchronizationState of this custom timestep */
	ECustomTimeStepSynchronizationState State = ECustomTimeStepSynchronizationState::Closed;

	bool bIgnoreWarningForOneFrame = true;

	/** Frame number tracking */
	bool bIsPreviousFrameNumberValid = false;
	uint64 PreviousFrameNumber = 0;
	uint64 DeltaFrameNumber = 0;

	/** Frame boundary monitoring ID */
	FGuid BoundaryMonitorListener;

	/** Cached frame rate used for boundary monitoring */
	FFrameRate CachedFrameRate;
};
