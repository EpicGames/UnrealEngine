// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaCapture.h"

#include "IRivermaxOutputStream.h"
#include "Misc/FrameRate.h"
#include "RivermaxMediaOutput.h"
#include "RivermaxTypes.h"

#include "RivermaxMediaCapture.generated.h"


class URivermaxMediaOutput;

/**
 * Output Media for Rivermax streams.
 */
UCLASS(BlueprintType)
class RIVERMAXMEDIA_API URivermaxMediaCapture : public UMediaCapture, public UE::RivermaxCore::IRivermaxOutputStreamListener
{
	GENERATED_BODY()

public:
	/** Rivermax capture specific API to provide stream options access */
	UE::RivermaxCore::FRivermaxOutputOptions GetOutputOptions() const;

	/** Returns information about last presented frame on the output stream */
	void GetLastPresentedFrameInformation(UE::RivermaxCore::FPresentedFrameInfo& OutFrameInfo) const;

	/** In cases of GPUDirect we should manually wait for work to be done. */
	virtual void WaitForGPU(FRHITexture* InRHITexture) override;

public:

	//~ Begin UMediaCapture interface
	virtual bool HasFinishedProcessing() const override;
protected:
	virtual bool ValidateMediaOutput() const override;
	virtual bool InitializeCapture() override;
	virtual bool UpdateSceneViewportImpl(TSharedPtr<FSceneViewport>& InSceneViewport) override;
	virtual bool UpdateRenderTargetImpl(UTextureRenderTarget2D* InRenderTarget) override;
	virtual void StopCaptureImpl(bool bAllowPendingFrameToBeProcess) override;
	virtual bool ShouldCaptureRHIResource() const override;
	virtual void OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow) override;
	virtual void OnRHIResourceCaptured_RenderingThread(FRHICommandListImmediate& RHICmdList, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FBufferRHIRef InBuffer) override;
	virtual void OnRHIResourceCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FBufferRHIRef InBuffer) override;
	virtual void OnFrameCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, const FMediaCaptureResourceData& InResourceData) override;
	virtual bool SupportsAnyThreadCapture() const override
	{
		return true;
	}
	virtual bool SupportsAutoRestart() const
	{
		return true;
	}

	/** For custom conversion, methods that need to be overriden */
	virtual FIntPoint GetCustomOutputSize(const FIntPoint& InSize) const override;
	virtual EMediaCaptureResourceType GetCustomOutputResourceType() const override;
	virtual FRDGBufferDesc GetCustomBufferDescription(const FIntPoint& InDesiredSize) const override;
	virtual void OnCustomCapture_RenderingThread(FRDGBuilder& GraphBuilder, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FRDGTextureRef InSourceTexture, FRDGBufferRef OutputBuffer, const FRHICopyTextureInfo& CopyInfo, FVector2D CropU, FVector2D CropV) override;
	virtual bool IsOutputSynchronizationSupported() const override { return true; }

#if WITH_EDITOR
	virtual FString GetAnalyticsEventPrefix() const override;
	virtual void GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const override;
#endif
	//~ End UMediaCapture interface

	//~ Begin UObject interface
	virtual bool IsReadyForFinishDestroy();
	//~ End UObject interface

	//~ Begin IRivermaxOutputStreamListener interface
	virtual void OnInitializationCompleted(bool bHasSucceed) override;
	virtual void OnStreamError() override;
	virtual void OnPreFrameEnqueue() override;
	//~ End IRivermaxOutputStreamListener interface

private:
	struct FRivermaxCaptureSyncData;

	/** Initializes capture and launches stream creation. */
	bool Initialize(const UE::RivermaxCore::FRivermaxOutputOptions& InMediaOutput);

	/**
	 * Checks whether the provided options are valid to start the capture.
	 * Resolves the interface address and ensures the resolution is aligned to the nearest pixel group.
	 * Returns false if the stream cannot be configured based on the provided settings.
	 */
	bool ConfigureCapture() const;

	/** Enqueues a RHI lambda to reserve a spot for the next frame to capture */
	void AddFrameReservationPass(FRDGBuilder& GraphBuilder);

	/** Common method called for non gpudirect route when a frame is captured, either from render thread or any thread  */
	void OnFrameCapturedInternal_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow);

	/** Common method called for gpudirect route when a frame is captured, either from render thread or any thread  */
	void OnRHIResourceCapturedInternal_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FBufferRHIRef InBuffer);

private:

	TStaticArray<TUniquePtr<UE::RivermaxCore::IRivermaxOutputStream>, static_cast<uint32>(UE::RivermaxCore::ERivermaxStreamType::MAX)> Streams;
	/** Set of options used to configure output stream */
	UE::RivermaxCore::FRivermaxOutputOptions Options;

	/** When using GPUDirect we have to rely on our own fence to wait for output to be converted into buffer. This will be waited for in WaitForGPU function. */
	TRefCountPtr<FRHIGPUFence> ShaderCompletedRenderingFence;

	/** Used with GPUDirect An Event used to block from render thread writing a fence again when we wait for it. */
	TSharedPtr<FEvent, ESPMode::ThreadSafe> GPUWaitCompleteEvent;

	/** Whether capture is active. Used to be queried by sync task */
	std::atomic<bool> bIsActive;
};
