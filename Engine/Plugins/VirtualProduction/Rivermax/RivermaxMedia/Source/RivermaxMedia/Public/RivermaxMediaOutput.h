// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaOutput.h"

#include "MediaIOCoreDefinitions.h"
#include "RivermaxMediaTypes.h"
#include "UObject/ObjectMacros.h"

#include "RivermaxMediaOutput.generated.h"



UENUM()
enum class ERivermaxMediaAlignmentMode : uint8
{
	/** 
	 * Uses NVIDIA Rivermax clock to calculate alignment points based on ST2059 
	 */
	AlignmentPoint,

	/** 
	 * Aligns frame scheduling with frame creation not going faster than frame interval 
	 * In its current shape, useful for a faster stream than frame creation rate
	 */
	FrameCreation,
};

/** 
 * Controls how rivermax capture behaves when there are no buffers available to capture into
 */
UENUM()
enum class ERivermaxFrameLockingMode : uint8
{
	/** If no frame available, continue */
	FreeRun,

	/** Blocks RHI thread prior to capture the current frame if no space is available. */
	BlockOnReservation,
};


/**
 * Output information for a Rivermax media capture.
 */
UCLASS(BlueprintType, meta=(MediaIOCustomLayout="Rivermax", DisplayName = "NVIDIA Rivermax Output"))
class RIVERMAXMEDIA_API URivermaxMediaOutput : public UMediaOutput
{
	GENERATED_BODY()

public:
	
	//~ Begin UMediaOutput interface
	virtual bool Validate(FString& FailureReason) const override;
	virtual FIntPoint GetRequestedSize() const override;
	virtual EPixelFormat GetRequestedPixelFormat() const override;
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override;
	
#if WITH_EDITOR
	virtual FString GetDescriptionString() const override;
	virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const override;
#endif //WITH_EDITOR

public:

	/** Saves SDP into the file fath. */
	void ExportSDP(const FString& InPath);

	/** Aggregates URivermaxMediaOutput settings into FRivermaxOutputOptions that have information about individual stream types. */
	UE::RivermaxCore::FRivermaxOutputOptions GenerateStreamOptions() const;

protected:
	virtual UMediaCapture* CreateMediaCaptureImpl() override;
	//~ End UMediaOutput interface


public:
	//~ UObject interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject interface

	/** Used by frame scheduler to know how to align the output */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Output")
	ERivermaxMediaAlignmentMode AlignmentMode = ERivermaxMediaAlignmentMode::AlignmentPoint;

	/** 
	 * Whether to produce a continuous output stream repeating last frame if no new frames provided 
	 * Note: Not supported in frame creation mode
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Output", meta = (EditCondition = "AlignmentMode != ERivermaxMediaAlignmentMode::FrameCreation"))
	bool bDoContinuousOutput = true;

	/** For alignment point mode, controls whether we stall engine before capturing if there are no buffer available to capture into */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Output")
	ERivermaxFrameLockingMode FrameLockingMode = ERivermaxFrameLockingMode::FreeRun;

	/** DEPRECATED: Formally controlled the number of frames that could be queued / used in output queue. */
	UE_DEPRECATED(5.8, "PresentationQueueSize is deprecated in favor of NumberOfTextureBuffers from UMediaOutput.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use UMediaOutput::NumberOfTextureBuffers instead."))
	int32 PresentationQueueSize_DEPRECATED = 2;

	/** 
	 * Experimental flag to use frame counter instead of using NVIDIA Rivermax clock for timestamping output frames
	 * Meant to be used for UE-UE streams where frame locking is done, e.g. nDisplay.
	 */
	UPROPERTY(EditAnywhere, Category = "Output", meta = (EditCondition = "AlignmentMode == ERivermaxMediaAlignmentMode::FrameCreation"))
	bool bDoFrameCounterTimestamping = true;


public:
	/** If false, use the default source buffer size. If true, a specific resolution will be used. */
	UE_DEPRECATED(5.7, "Override Resolution is deprecated. Use (Video Stream)->(Override Resolution) instead.")
	UPROPERTY()
	bool bOverrideResolution_DEPRECATED = false;

	/** Resolution of this output stream */
	UE_DEPRECATED(5.7, "Resolution is deprecated. Use (Video Stream)->(Resolution) instead.")
	UPROPERTY()
	FIntPoint Resolution_DEPRECATED = { 1920, 1080 };

	/** Frame rate of this output stream */
	UE_DEPRECATED(5.7, "Frame Rate is deprecated. Use (Video Stream)->(Frame Rate) instead.")
	UPROPERTY()
	FFrameRate FrameRate_DEPRECATED = { 24,1 };

	/** Pixel format for this output stream */
	UE_DEPRECATED(5.7, "Pixel Format is deprecated. Use (Video Stream)->(Pixel Format) instead.")
	UPROPERTY()
	ERivermaxPixelFormat PixelFormat_DEPRECATED = ERivermaxPixelFormat::RGB_10bit;

	UE_DEPRECATED(5.7, "Interface Address is deprecated. Use (Anc Stream) or (Video Stream)->(Interface Address) instead.")
	UPROPERTY()
	FString InterfaceAddress_DEPRECATED = TEXT("*.*.*.*");

	UE_DEPRECATED(5.7, "Stream Address is deprecated. Use (Anc Stream) or (Video Stream)->(Stream Address) instead.")
	UPROPERTY()
	FString StreamAddress_DEPRECATED = UE::RivermaxCore::DefaultStreamAddress;

	/** Port to use for this output */
	UE_DEPRECATED(5.7, "Port is deprecated. Use (Anc Stream) or (Video Stream)->(Port) instead.")
	UPROPERTY()
	int32 Port_DEPRECATED = 50000;

	/** Whether to use GPUDirect if available (Memcopy from GPU to NIC directly bypassing system) if available */
	UE_DEPRECATED(5.7, "Use GPU Direct is deprecated. Use Video->(GPU Direct) instead.")
	UPROPERTY()
	bool bUseGPUDirect_DEPRECATED = true;

	/** Video output stream properties. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Video")
	FRivermaxVideoStream VideoStream;

	/** Collection of Ancillary streams. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Anc")
	TArray<FRivermaxAncStream> AncStreams;

private:
	friend class URivermaxMediaCapture;
};
