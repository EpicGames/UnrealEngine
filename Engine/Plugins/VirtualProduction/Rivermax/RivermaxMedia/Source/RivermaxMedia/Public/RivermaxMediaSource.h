// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureCardMediaSource.h"

#include "MediaIOCoreDefinitions.h"
#include "RivermaxMediaTypes.h"

#include "RivermaxMediaSource.generated.h"


/**
 * Media source for Rivermax streams.
 */
UCLASS(BlueprintType, hideCategories=(Platforms,Object), meta=(MediaIOCustomLayout="Rivermax", DisplayName = "NVIDIA Rivermax Source"))
class RIVERMAXMEDIA_API URivermaxMediaSource : public UCaptureCardMediaSource
{
	GENERATED_BODY()

public:

	URivermaxMediaSource();

public:

	/** Video input stream properties. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Video")
	FRivermaxVideoStream VideoStream;

	/** ANC (ST 2110-40) input streams. Add entries to receive ancillary data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ANC")
	TArray<FRivermaxAncStream> AncStreams;

	/** If false, use the default source buffer size. If true, a specific resolution will be used. */
	UE_DEPRECATED(5.8, "Override Resolution is deprecated. Use (Video Stream)->(Override Resolution) instead.")
	UPROPERTY()
	bool bOverrideResolution = false;

	/** Incoming stream video resolution */
	UE_DEPRECATED(5.8, "Resolution is deprecated. Use (Video Stream)->(Resolution) instead.")
	UPROPERTY()
	FIntPoint Resolution = {1920, 1080};

	/** Incoming stream video frame rate */
	UE_DEPRECATED(5.8, "Frame Rate is deprecated. Use (Video Stream)->(Frame Rate) instead.")
	UPROPERTY()
	FFrameRate FrameRate = {24,1};

	/** Incoming stream pixel format */
	UE_DEPRECATED(5.8, "Pixel Format is deprecated. Use (Video Stream)->(Pixel Format) instead.")
	UPROPERTY()
	ERivermaxPixelFormat PixelFormat = ERivermaxPixelFormat::RGB_10bit;

	/** Network card interface to use to receive data */
	UE_DEPRECATED(5.8, "Interface Address is deprecated. Use (Video Stream)->(Interface Address) instead.")
	UPROPERTY()
	FString InterfaceAddress = TEXT("*.*.*.*");

	/** IP address where incoming stream is coming from */
	UE_DEPRECATED(5.8, "Stream Address is deprecated. Use (Video Stream)->(Stream Address) instead.")
	UPROPERTY()
	FString StreamAddress = UE::RivermaxCore::DefaultStreamAddress;

	/** Port used by the sender to send its stream */
	UE_DEPRECATED(5.8, "Port is deprecated. Use (Video Stream)->(Port) instead.")
	UPROPERTY()
	int32 Port = 50000;

	/** Whether to use GPUDirect if available */
	UE_DEPRECATED(5.8, "Use GPU Direct is deprecated. Use (Video Stream)->(GPU Direct) instead.")
	UPROPERTY()
	bool bUseGPUDirect = true;

public:
	//~ Begin IMediaOptions interface
	virtual bool GetMediaOption(const FName& Key, bool DefaultValue) const override;
	virtual int64 GetMediaOption(const FName& Key, int64 DefaultValue) const override;
	virtual FString GetMediaOption(const FName& Key, const FString& DefaultValue) const override;
	virtual bool HasMediaOption(const FName& Key) const override;
	//~ End IMediaOptions interface

	virtual void PostLoad() override;
	void Serialize(FArchive& Ar) override;

public:
	//~ Begin UMediaSource interface
	virtual FString GetUrl() const override;
	virtual bool Validate() const override;

#if WITH_EDITOR
	virtual FString GetDescriptionString() const override;
	virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const override;
#endif //WITH_EDITOR
	
	//~ End UMediaSource interface
};
