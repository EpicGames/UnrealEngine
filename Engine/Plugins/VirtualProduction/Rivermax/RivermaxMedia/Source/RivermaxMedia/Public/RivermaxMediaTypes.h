// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RivermaxTypes.h"

#include "RivermaxMediaTypes.generated.h"

/** Base struct shared by all Rivermax stream config structs (output and input). */
USTRUCT(BlueprintType)
struct RIVERMAXMEDIA_API FRivermaxStream
{
	GENERATED_BODY()

	/**
	 * Network card interface to use.
	 * Wildcards are supported: 192.*.0.110, 192.168.0.1?0, 192.168.0.1*
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FString InterfaceAddress = TEXT("*.*.*.*");

	/** Multicast IP address of the stream */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FString StreamAddress = UE::RivermaxCore::DefaultStreamAddress;

	/** UDP port of the stream */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	int32 Port = 50000;
};

/** 
* ANC stream type. None disables the stream.
* Anc specific reflection of UE::RivermaxCore::ERivermaxStreamType, 
* for ease of access ST2110_40_TC starts at 3 to reflect ERivermaxStreamType.
* 	ST2110_20 = 0,
*	ST2110_30 = 1,
*	ST2110_40 = 2, - general ANC
*/
UENUM(BlueprintType)
enum class ERivermaxAncStreamType : uint8
{
	None         = 0  UMETA(DisplayName = "None"),
	ST2110_40_TC = 3  UMETA(DisplayName = "ST 2110-40 (Timecode)"),
};

/** All settings required to initialize an ANC (2110-40) stream (input or output). */
USTRUCT(BlueprintType)
struct RIVERMAXMEDIA_API FRivermaxAncStream : public FRivermaxStream
{
	GENERATED_BODY()

	/** ANC stream type such as Timecode. Set to None to disable. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	ERivermaxAncStreamType StreamType = ERivermaxAncStreamType::None;
};

/** Pixel format used by ST 2110-20 video streams (input and output). */
UENUM(BlueprintType)
enum class ERivermaxPixelFormat : uint8
{
	YUV422_8bit     UMETA(DisplayName = "8bit YUV422"),
	YUV422_10bit    UMETA(DisplayName = "10bit YUV422"),
	RGB_8bit        UMETA(DisplayName = "8bit RGB"),
	RGB_10bit       UMETA(DisplayName = "10bit RGB"),
	RGB_12bit       UMETA(DisplayName = "12bit RGB"),
	RGB_16bit_Float UMETA(DisplayName = "16bit Float RGB"),
};

/** Shared settings for an ST 2110-20 video stream (input or output). */
USTRUCT(BlueprintType)
struct RIVERMAXMEDIA_API FRivermaxVideoStream : public FRivermaxStream
{
	GENERATED_BODY()

	/** Frame rate of this stream */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FFrameRate FrameRate = {24, 1};

	/** If false, use the size provided by the source. If true, a specific resolution will be used. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (InlineEditConditionToggle))
	bool bOverrideResolution = false;

	/** Stream resolution */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition = "bOverrideResolution"))
	FIntPoint Resolution = {1920, 1080};

	/** Pixel format */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	ERivermaxPixelFormat PixelFormat = ERivermaxPixelFormat::RGB_10bit;

	/** Whether to use GPUDirect if available (memory transfer bypassing system memory) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Settings")
	bool bUseGPUDirect = true;
};
