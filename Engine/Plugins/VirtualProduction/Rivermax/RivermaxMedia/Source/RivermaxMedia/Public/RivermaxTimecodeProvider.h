// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenlockedTimecodeProvider.h"

#include "RivermaxTimecodeProvider.generated.h"


class UEngine;


/**
 * Timecode provider based on time returned by Rivermax GetTime API
 * Rivermax clock is configured using Project Setting.
 * If using multiple interfaces, make sure to configure the interface address you wish to use.
 * 
 * PTP time is returned in International Atomic Time referential (TAI) as opposed to Coordinated Universal Time (UTC)
 * Timecode going out of the clock generator is generally using UTC time reference so to generate an engine timecode
 * that aligns with devices receiving LTC timecode, we need to convert PTP TAI time to UTC time. 
 * At the moment, that offset is fixed at 37 seconds. 
 * 
 * Clock generators can also output timecode with daylight saving time (DST) offsets which would not be affecting PTP time reference
 * If devices receive timecode with DST offsets, engine timecode will need to be adjusted to correctly align.
 */
UCLASS(Blueprintable, editinlinenew)
class RIVERMAXMEDIA_API URivermaxTimecodeProvider : public UGenlockedTimecodeProvider
{
public:
	GENERATED_BODY()

	//~ UTimecodeProvider interface
	virtual bool FetchTimecode(FQualifiedFrameTime& OutFrameTime) override;
	virtual ETimecodeProviderSynchronizationState GetSynchronizationState() const override { return State; }
	virtual bool Initialize(class UEngine* InEngine) override;
	virtual void Shutdown(class UEngine* InEngine) override;
	virtual bool SupportsAutoDetected() const override
	{
		return false;
	}

public:

	/** Frame Rate to provide timecode in */
	UPROPERTY(EditAnywhere, Category = "Genlock options")
	FFrameRate FrameRate = FFrameRate(24, 1);

	/** 
	 * Frame offset subtracted to converted timecode to match LTC timecode due to transportation of timecode
	 * For LTC, it takes a full frame to transport the new value but for PTP, this doesn't happen, time is local.
	 * If a timecoded video feed is coming into Unreal, if PTP timecode isn't offset, it will be leading by 1 frame.
	 * A value of 1 means PTP timecode is 1 frame ahead so we substract 1 frame to the result
	 * A value of -1 would mean PTP is late by one frame compared to LTC so we add a frame to the result.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Genlock options")
	int32 PTPToLTCTimecodeFrameOffset = 1;

	/** TAI to UTC time offset. TAI is currently 37 seconds ahead so we substract 37 seconds to PTP time to convert to UTC. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Genlock options")
	int32 UTCSecondsOffset = 37;

	/** 
	 * If timecode generated by the clock generator has daylight saving applied, add offset here in order to match PTP based timecode with devices getting LTC timecode 
	 * A positive value will be subtracted from the timecode generated from PTP time.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Genlock options")
	int32 DaylightSavingTimeHourOffset = 0;

private:

	/** The current SynchronizationState of the TimecodeProvider*/
	ETimecodeProviderSynchronizationState State = ETimecodeProviderSynchronizationState::Closed;
};
