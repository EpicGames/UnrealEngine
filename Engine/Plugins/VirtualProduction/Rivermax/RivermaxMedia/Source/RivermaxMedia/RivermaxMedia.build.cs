// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RivermaxMedia : ModuleRules
{
	public RivermaxMedia(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"MediaAssets",
				"MediaIOCore",
				"RivermaxCore",
				"TimeManagement"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Engine",
				"OpenColorIO",
				"RenderCore",
				"RHI",
				"RivermaxRendering"
			}
		);

		if (Target.bBuildEditor)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
				}
			);
		}
	}
}
