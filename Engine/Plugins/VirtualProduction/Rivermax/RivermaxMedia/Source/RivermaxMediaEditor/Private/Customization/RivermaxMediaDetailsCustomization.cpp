// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxMediaDetailsCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "IDesktopPlatform.h"
#include "RivermaxMediaOutput.h"
#include "Widgets/Input/SButton.h"


TSharedRef<IDetailCustomization> FRivermaxMediaDetailsCustomization::MakeInstance()
{
	return MakeShared<FRivermaxMediaDetailsCustomization>();
}

void FRivermaxMediaDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);

	for (int32 ObjectIndex = 0; ObjectIndex < CustomizedObjects.Num(); ++ObjectIndex)
	{
		// For now, take care of Rivermax source to customize their interface field
		const TWeakObjectPtr<UObject> Obj = CustomizedObjects[ObjectIndex];

		if (URivermaxMediaOutput* Output = Cast<URivermaxMediaOutput>(Obj.Get()))
		{
			IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("Advanced", FText::FromString("Advanced"), ECategoryPriority::Important);
			Category.AddCustomRow(FText::FromString("Export SDP"))
				.ValueContent()
				[
					SNew(SButton)
						.Text(FText::FromString("Export SDP"))
						.OnClicked(FOnClicked::CreateRaw(this, &FRivermaxMediaDetailsCustomization::OnExportSDP, Output))
				];
		}
	}
}

FReply FRivermaxMediaDetailsCustomization::OnExportSDP(URivermaxMediaOutput* Output)
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		TArray<FString> OutFilenames;
		const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		bool bSuccess = DesktopPlatform->SaveFileDialog(
			ParentWindowHandle,
			TEXT("Export SDP File"),
			FPaths::ProjectDir(),
			TEXT("Rivermax Media Output.sdp"),
			TEXT("SDP Files (*.sdp)|*.sdp"),
			EFileDialogFlags::None,
			OutFilenames
		);

		if (bSuccess && OutFilenames.Num() > 0)
		{
			Output->ExportSDP(OutFilenames[0]);
		}
	}
	return FReply::Handled();
}
