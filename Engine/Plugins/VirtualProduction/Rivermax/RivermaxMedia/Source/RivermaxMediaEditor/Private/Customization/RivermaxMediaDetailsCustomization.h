// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"

class IPropertyHandle;
class URivermaxMediaOutput;

class FRivermaxMediaDetailsCustomization : public IDetailCustomization
{
public:
	
	static TSharedRef<IDetailCustomization> MakeInstance();
	
	//~ Begin IDetailCustomization Interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface

private:

	/** Shows up a path selection dialog and exports SDP there. Currently only works in Media Output and not in ndisplay. */
	FReply OnExportSDP(URivermaxMediaOutput* Output);
};
