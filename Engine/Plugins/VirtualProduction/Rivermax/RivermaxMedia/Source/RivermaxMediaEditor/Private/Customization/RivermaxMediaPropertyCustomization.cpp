// Copyright Epic Games, Inc. All Rights Reserved.
#include "RivermaxMediaPropertyCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Customizations/RivermaxDeviceSelectionCustomization.h"
#include "RivermaxMediaOutput.h"


void FRivermaxStreamCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> StreamTypeHandle = StructHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRivermaxAncStream, StreamType));

	FText HeaderText;
	if (StreamTypeHandle.IsValid())
	{
		FString EnumValue;
		if (StreamTypeHandle->GetValueAsFormattedString(EnumValue) == FPropertyAccess::Success)
		{
			HeaderText = FText::FromString(EnumValue);
		}
		else
		{
			HeaderText = StreamTypeHandle->GetPropertyDisplayName();
		}
	}
	else
	{
		// FRivermaxVideoStream has no StreamType child.
		HeaderText = FText::FromString(TEXT("Video Stream"));
	}

	HeaderRow.NameContent()
	[
		SNew(STextBlock)
			.Text(HeaderText)
			.Font(IDetailLayoutBuilder::GetDetailFontBold())
	];
}

void FRivermaxStreamCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	uint32 NumChildren = 0;
	StructHandle->GetNumChildren(NumChildren);
	FName InterfaceAddressName = GET_MEMBER_NAME_CHECKED(FRivermaxStream, InterfaceAddress);
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> Child = StructHandle->GetChildHandle(ChildIndex);
		if (!Child.IsValid() || !Child->GetProperty())
		{
			continue;
		}

		if (Child->GetProperty()->GetFName() == InterfaceAddressName)
		{
			FString InitialValue;
			if (Child->GetValue(InitialValue) != FPropertyAccess::Success)
			{
				Child->GetValueAsFormattedString(InitialValue);
			}

			UE::RivermaxCore::Utils::AddInterfaceAddressRow(
				ChildBuilder,
				Child.ToSharedRef(),
				InitialValue,
				CustomizationUtils);
		}
		else
		{
			ChildBuilder.AddProperty(Child.ToSharedRef());
		}
	}
}