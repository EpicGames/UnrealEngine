// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularFeatures/RivermaxMediaInitializerFeature.h"

#include "IRivermaxCoreModule.h"
#include "IRivermaxManager.h"

#include "RivermaxMediaSource.h"
#include "RivermaxMediaOutput.h"


bool FRivermaxMediaInitializerFeature::IsMediaObjectSupported(const UObject* MediaObject)
{
	if (MediaObject)
	{
		return MediaObject->IsA<URivermaxMediaSource>() || MediaObject->IsA<URivermaxMediaOutput>();
	}

	return false;
}

bool FRivermaxMediaInitializerFeature::AreMediaObjectsCompatible(const UObject* MediaSource, const UObject* MediaOutput)
{
	if (MediaSource && MediaOutput)
	{
		return MediaSource->IsA<URivermaxMediaSource>() && MediaOutput->IsA<URivermaxMediaOutput>();
	}

	return false;
}

bool FRivermaxMediaInitializerFeature::GetSupportedMediaPropagationTypes(const UObject* MediaSource, const UObject* MediaOutput, EMediaStreamPropagationType& OutPropagationTypes)
{
	if (!IsMediaObjectSupported(MediaSource) ||
		!IsMediaObjectSupported(MediaOutput) ||
		!AreMediaObjectsCompatible(MediaSource, MediaOutput))
	{
		return false;
	}

	OutPropagationTypes =
		EMediaStreamPropagationType::LocalUnicast |
		EMediaStreamPropagationType::LocalMulticast |
		EMediaStreamPropagationType::Unicast |
		EMediaStreamPropagationType::Multicast;

	return true;
}

void FRivermaxMediaInitializerFeature::InitializeMediaObjectForTile(UObject* MediaObject, const FMediaObjectOwnerInfo& OnwerInfo, const FIntPoint& TilePos)
{
	if (URivermaxMediaSource* RivermaxMediaSource = Cast<URivermaxMediaSource>(MediaObject))
	{
		RivermaxMediaSource->EvaluationType              = EMediaIOSampleEvaluationType::Timecode;
		RivermaxMediaSource->bFramelock                  = true;
		RivermaxMediaSource->bUseTimeSynchronization     = true;
		RivermaxMediaSource->FrameDelay                  = 0;
		RivermaxMediaSource->VideoStream.bOverrideResolution = false;
		RivermaxMediaSource->VideoStream.FrameRate       = { 60, 1 };
		RivermaxMediaSource->VideoStream.PixelFormat     = ERivermaxPixelFormat::RGB_10bit;
		RivermaxMediaSource->VideoStream.InterfaceAddress = GetRivermaxInterfaceAddress();
		RivermaxMediaSource->VideoStream.StreamAddress   = GenerateStreamAddress(OnwerInfo.OwnerUniqueIdx, TilePos);
		RivermaxMediaSource->VideoStream.Port            = 50000;
		RivermaxMediaSource->VideoStream.bUseGPUDirect   = true;
	}
	else if (URivermaxMediaOutput* RivermaxMediaOutput = Cast<URivermaxMediaOutput>(MediaObject))
	{
		RivermaxMediaOutput->AlignmentMode       = ERivermaxMediaAlignmentMode::FrameCreation;
		RivermaxMediaOutput->bDoContinuousOutput = false;
		RivermaxMediaOutput->FrameLockingMode    = ERivermaxFrameLockingMode::BlockOnReservation;
		RivermaxMediaOutput->NumberOfTextureBuffers = 2;
		RivermaxMediaOutput->bDoFrameCounterTimestamping = true;

		RivermaxMediaOutput->VideoStream.bOverrideResolution = false;
		RivermaxMediaOutput->VideoStream.FrameRate           = { 60,1 };
		RivermaxMediaOutput->VideoStream.PixelFormat         = ERivermaxPixelFormat::RGB_10bit;
		RivermaxMediaOutput->VideoStream.InterfaceAddress    = GetRivermaxInterfaceAddress();
		RivermaxMediaOutput->VideoStream.StreamAddress       = GenerateStreamAddress(OnwerInfo.OwnerUniqueIdx, TilePos);
		RivermaxMediaOutput->VideoStream.Port                = 50000;
		RivermaxMediaOutput->VideoStream.bUseGPUDirect       = true;
	}
}

void FRivermaxMediaInitializerFeature::InitializeMediaObjectForFullFrame(UObject* MediaObject, const FMediaObjectOwnerInfo& OnwerInfo)
{
	if (URivermaxMediaSource* RivermaxMediaSource = Cast<URivermaxMediaSource>(MediaObject))
	{
		RivermaxMediaSource->EvaluationType              = EMediaIOSampleEvaluationType::Timecode;
		RivermaxMediaSource->bFramelock                  = true;
		RivermaxMediaSource->bUseTimeSynchronization     = true;
		RivermaxMediaSource->FrameDelay                  = 0;
		RivermaxMediaSource->VideoStream.bOverrideResolution = false;
		RivermaxMediaSource->VideoStream.FrameRate       = { 60, 1 };
		RivermaxMediaSource->VideoStream.PixelFormat     = ERivermaxPixelFormat::RGB_10bit;
		RivermaxMediaSource->VideoStream.InterfaceAddress = GetRivermaxInterfaceAddress();
		RivermaxMediaSource->VideoStream.StreamAddress   = GenerateStreamAddress(OnwerInfo.ClusterNodeUniqueIdx.Get(0), OnwerInfo.OwnerUniqueIdx, OnwerInfo.OwnerType);
		RivermaxMediaSource->VideoStream.Port            = 50000;
		RivermaxMediaSource->VideoStream.bUseGPUDirect   = true;
	}
	else if (URivermaxMediaOutput* RivermaxMediaOutput = Cast<URivermaxMediaOutput>(MediaObject))
	{
		RivermaxMediaOutput->FrameLockingMode = ERivermaxFrameLockingMode::BlockOnReservation;
		RivermaxMediaOutput->NumberOfTextureBuffers = 2;
		RivermaxMediaOutput->VideoStream.bOverrideResolution   = false;
		RivermaxMediaOutput->VideoStream.PixelFormat           = ERivermaxPixelFormat::RGB_10bit;
		RivermaxMediaOutput->VideoStream.InterfaceAddress      = GetRivermaxInterfaceAddress();
		RivermaxMediaOutput->VideoStream.StreamAddress         = GenerateStreamAddress(OnwerInfo.ClusterNodeUniqueIdx.Get(0), OnwerInfo.OwnerUniqueIdx, OnwerInfo.OwnerType);
		RivermaxMediaOutput->VideoStream.Port                  = 50000;

		switch (OnwerInfo.OwnerType)
		{
		case FMediaObjectOwnerInfo::EMediaObjectOwnerType::ICVFXCamera:
		case FMediaObjectOwnerInfo::EMediaObjectOwnerType::Viewport:
			RivermaxMediaOutput->AlignmentMode = ERivermaxMediaAlignmentMode::FrameCreation;
			RivermaxMediaOutput->bDoContinuousOutput = false;
			RivermaxMediaOutput->bDoFrameCounterTimestamping = true;
			RivermaxMediaOutput->VideoStream.FrameRate           = { 60,1 };
			RivermaxMediaOutput->VideoStream.bUseGPUDirect       = true;
			break;

		case FMediaObjectOwnerInfo::EMediaObjectOwnerType::Backbuffer:
			RivermaxMediaOutput->AlignmentMode = ERivermaxMediaAlignmentMode::AlignmentPoint;
			RivermaxMediaOutput->bDoContinuousOutput = true;
			RivermaxMediaOutput->bDoFrameCounterTimestamping = false;
			RivermaxMediaOutput->VideoStream.FrameRate           = { 24,1 };
			RivermaxMediaOutput->VideoStream.bUseGPUDirect       = false;
			break;

		default:
			checkNoEntry();
		}
	}
}

FString FRivermaxMediaInitializerFeature::GetRivermaxInterfaceAddress() const
{
	FString ResultAddress{ TEXT("*.*.*.*") };

	// Now let's see if we have any interfaces available
	IRivermaxCoreModule& RivermaxModule = FModuleManager::LoadModuleChecked<IRivermaxCoreModule>(TEXT("RivermaxCore"));
	const TConstArrayView<UE::RivermaxCore::FRivermaxDeviceInfo> Devices = RivermaxModule.GetRivermaxManager()->GetDevices();
	if (Devices.Num() > 0)
	{
		// Split address into octets
		TArray<FString> Octets;
		Devices[0].InterfaceAddress.ParseIntoArray(Octets, TEXT("."));

		// IPv4 always has 4 octets
		if (Octets.Num() == 4)
		{
			ResultAddress = FString::Printf(TEXT("%s.%s.%s.*"), *Octets[0], *Octets[1], *Octets[2]);
		}
	}

	return ResultAddress;
}

FString FRivermaxMediaInitializerFeature::GenerateStreamAddress(uint8 OwnerUniqueIdx, const FIntPoint& TilePos) const
{
	static const constexpr uint8 MaxVal = TNumericLimits<uint8>::Max();
	checkSlow(TilePos.X < MaxVal && TilePos.Y < MaxVal);

	static constexpr uint8 AddressOffsetForTiles = 200;
	checkSlow((AddressOffsetForTiles + OwnerUniqueIdx) < MaxVal);

	// 228.200.*.* - 228.255.*.* - range for tiled media (max 56 objects)
	return FString::Printf(TEXT("228.%u.%u.%u"), AddressOffsetForTiles + OwnerUniqueIdx, TilePos.X, TilePos.Y);
}

FString FRivermaxMediaInitializerFeature::GenerateStreamAddress(uint8 ClusterNodeUniqueIdx, uint8 OwnerUniqueIdx, const FMediaObjectOwnerInfo::EMediaObjectOwnerType OwnerType) const
{
	// 228.0.*.* - 228.199.*.* - range for full-frame media (max 200 objects). But could be extended up to the limit if no tiles used.
	return FString::Printf(TEXT("228.%u.%u.%u"), ClusterNodeUniqueIdx, static_cast<uint8>(OwnerType), OwnerUniqueIdx);
}
