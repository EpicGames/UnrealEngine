// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDisplayClusterModularFeatureMediaInitializer.h"


/**
 * Rivermax media source/output initializer for nDisplay
 */
class FRivermaxMediaInitializerFeature
	: public IDisplayClusterModularFeatureMediaInitializer
{
public:

	//~ Begin IDisplayClusterModularFeatureMediaInitializer
	virtual bool IsMediaObjectSupported(const UObject* MediaObject) override;
	virtual bool AreMediaObjectsCompatible(const UObject* MediaSource, const UObject* MediaOutput) override;
	virtual bool GetSupportedMediaPropagationTypes(const UObject* MediaSource, const UObject* MediaOutput, EMediaStreamPropagationType& OutPropagationTypes) override;
	virtual void InitializeMediaObjectForTile(UObject* MediaObject, const FMediaObjectOwnerInfo& OnwerInfo, const FIntPoint& TilePos) override;
	virtual void InitializeMediaObjectForFullFrame(UObject* MediaObject, const FMediaObjectOwnerInfo& OnwerInfo) override;
	//~ End IDisplayClusterModularFeatureMediaInitializer

private:

	/** Returns first interface address if available */
	FString GetRivermaxInterfaceAddress() const;

	/** Generates stream address based on the function parameters */
	FString GenerateStreamAddress(uint8 OwnerUniqueIdx, const FIntPoint& TilePos) const;

	/** Generates stream address based on the function parameters */
	FString GenerateStreamAddress(uint8 ClusterNodeUniqueIdx, uint8 OwnerUniqueIdx, const FMediaObjectOwnerInfo::EMediaObjectOwnerType OwnerType) const;
};
