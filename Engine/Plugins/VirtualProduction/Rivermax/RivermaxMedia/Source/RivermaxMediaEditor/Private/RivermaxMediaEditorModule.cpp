// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "PropertyEditorModule.h"
#include "RivermaxMediaOutput.h"
#include "RivermaxMediaSource.h"
#include "Customization/RivermaxMediaDetailsCustomization.h"
#include "Features/IModularFeatures.h"
#include "ModularFeatures/RivermaxMediaInitializerFeature.h"
#include "Customization/RivermaxMediaPropertyCustomization.h"

class FRivermaxMediaEditorModule : public IModuleInterface
{
public:

	//~ Begin IModuleInterface
	virtual void StartupModule() override
	{
		RegisterCustomizations();
		RegisterModularFeatures();
	}
	
	virtual void ShutdownModule() override
	{
		UnregisterCustomizations();
		UnregisterModularFeatures();
	}
	//~End IModuleInterface

private:
	/** Registers detail customization for Rivermax media types */
	void RegisterCustomizations()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		const auto AddCustomization = [&PropertyModule, this](const FName ClassName, FOnGetDetailCustomizationInstance Customizor)
		{
			PropertyModule.RegisterCustomClassLayout(ClassName, Customizor);
			CustomizedClasses.Add(ClassName);
		};

		AddCustomization(URivermaxMediaSource::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FRivermaxMediaDetailsCustomization::MakeInstance));

		PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("RivermaxStream"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRivermaxStreamCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("RivermaxVideoStream"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRivermaxStreamCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("RivermaxAncStream"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRivermaxStreamCustomization::MakeInstance));
		PropertyModule.NotifyCustomizationModuleChanged();
	}

	/** Unregisters registered customizations */
	void UnregisterCustomizations()
	{
		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
		if (PropertyModule)
		{
			for (FName ClassName : CustomizedClasses)
			{
				PropertyModule->UnregisterCustomClassLayout(ClassName);
			}

			PropertyModule->UnregisterCustomPropertyTypeLayout(TEXT("RivermaxStream"));
			PropertyModule->UnregisterCustomPropertyTypeLayout(TEXT("RivermaxVideoStream"));
			PropertyModule->UnregisterCustomPropertyTypeLayout(TEXT("RivermaxAncStream"));
			PropertyModule->NotifyCustomizationModuleChanged();
		}
	}

	/** Registers modular features for external modules */
	void RegisterModularFeatures()
	{
		// Instantiate modular features
		MediaInitializer = MakeUnique<FRivermaxMediaInitializerFeature>();

		// Register modular features
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		ModularFeatures.RegisterModularFeature(FRivermaxMediaInitializerFeature::ModularFeatureName, MediaInitializer.Get());
	}

	/** Unregisters modular features */
	void UnregisterModularFeatures()
	{
		// Unregister modular features
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		ModularFeatures.UnregisterModularFeature(FRivermaxMediaInitializerFeature::ModularFeatureName, MediaInitializer.Get());
	}

private:
	TArray<FName> CustomizedClasses;

	/** MediaInitializer modular feature instance */
	TUniquePtr<FRivermaxMediaInitializerFeature> MediaInitializer;
};


IMPLEMENT_MODULE(FRivermaxMediaEditorModule, RivermaxMediaEditor);
