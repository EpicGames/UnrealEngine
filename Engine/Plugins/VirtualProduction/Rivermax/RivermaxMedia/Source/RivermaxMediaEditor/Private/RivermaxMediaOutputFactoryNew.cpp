// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxMediaOutputFactoryNew.h"

#include "AssetTypeCategories.h"
#include "RivermaxMediaOutput.h"
#include "EngineAnalytics.h"


/* UAjaMediaSourceFactoryNew structors
 *****************************************************************************/

URivermaxMediaOutputFactoryNew::URivermaxMediaOutputFactoryNew()
{
	SupportedClass = URivermaxMediaOutput::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

/**
 * @EventName MediaFramework.RivermaxMediaOutputCreated
 * @Trigger Triggered when an Rivermax media output asset is created.
 * @Type Client
 * @Owner MediaIO Team
 */
UObject* URivermaxMediaOutputFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.RivermaxMediaOutputCreated"));
	}
	
	return NewObject<URivermaxMediaOutput>(InParent, InClass, InName, Flags);
}


uint32 URivermaxMediaOutputFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool URivermaxMediaOutputFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}
