// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "UObject/ObjectMacros.h"

#include "RivermaxMediaOutputFactoryNew.generated.h"


/**
 * Implements a factory for URivermaxMediaOutput objects.
 */
UCLASS(hidecategories=Object)
class URivermaxMediaOutputFactoryNew : public UFactory
{
	GENERATED_BODY()

public:

	URivermaxMediaOutputFactoryNew();

	//~ UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
};
