// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxMediaSourceFactoryNew.h"

#include "AssetTypeCategories.h"
#include "EngineAnalytics.h"
#include "RivermaxMediaSource.h"


/* URivermaxMediaSourceFactoryNew structors
 *****************************************************************************/

URivermaxMediaSourceFactoryNew::URivermaxMediaSourceFactoryNew()
{
	SupportedClass = URivermaxMediaSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

 /**
 * @EventName MediaFramework.RivermaxMediaSourceCreated
 * @Trigger Triggered when a Rivermax media source asset is created.
 * @Type Client
 * @Owner MediaIO Team
 */
UObject* URivermaxMediaSourceFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.RivermaxMediaSourceCreated"));
	}

	return NewObject<URivermaxMediaSource>(InParent, InClass, InName, Flags);
}


uint32 URivermaxMediaSourceFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool URivermaxMediaSourceFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}
