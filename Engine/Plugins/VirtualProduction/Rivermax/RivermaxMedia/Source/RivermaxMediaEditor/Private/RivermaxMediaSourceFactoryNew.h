// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "RivermaxMediaSourceFactoryNew.generated.h"


/**
 * Implements a factory for URivermaxMediaSource objects.
 */
UCLASS(hidecategories=Object)
class URivermaxMediaSourceFactoryNew
	: public UFactory
{
	GENERATED_BODY()

public:

	URivermaxMediaSourceFactoryNew();

	//~ UFactory Interface

	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
};
