// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class RivermaxMediaEditor : ModuleRules
{
	public RivermaxMediaEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetDefinition",
				"AssetTools",
				"Core",
				"CoreUObject",
				"DisplayClusterModularFeaturesEditor",
				"EditorStyle",
				"Engine",
				"InputCore",
				"MediaAssets",
				"MediaIOCore",
				"MediaIOEditor",
				"MediaPlayerEditor",
				"RivermaxCore",
				"RivermaxEditor",
				"RivermaxMedia",
				"Slate",
				"SlateCore",
				"UnrealEd"
			}
		);
	}
}
