// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "IMediaPlayerFactory.h"
#include "IMediaModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
#endif

#include "IRivermaxMediaModule.h"


#define LOCTEXT_NAMESPACE "RivermaxMediaFactoryModule"


/**
 * Implements the RivermaxMediaFactory module.
 */
class FRivermaxMediaFactoryModule
	: public IMediaPlayerFactory
	, public IModuleInterface
{
public:

	//~ IMediaPlayerFactory interface

	bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
	}

	int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* /*Options*/, TArray<FText>* /*OutWarnings*/, TArray<FText>* OutErrors) const override
	{
		FString Scheme;
		FString Location;

		// check scheme
		if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
			}

			return 0;
		}

		if (!SupportedUriSchemes.Contains(Scheme))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
			}

			return 0;
		}

		return 100;
	}

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		IRivermaxMediaModule* RivermaxMediaModule = FModuleManager::LoadModulePtr<IRivermaxMediaModule>("RivermaxMedia");
		return (RivermaxMediaModule != nullptr) ? RivermaxMediaModule->CreatePlayer(EventSink) : nullptr;
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("MediaPlayerDisplayName", "Rivermax Device Interface");
	}

	virtual FName GetPlayerName() const override
	{
		static FName PlayerName(TEXT("RivermaxMedia"));
		return PlayerName;
	}

	virtual FGuid GetPlayerPluginGUID() const override
	{
		static FGuid PlayerPluginGUID(0xF537595A, 0x8E8D452B, 0xB8C05707, 0x6B334234);
		return PlayerPluginGUID;
	}

	virtual const TArray<FString>& GetSupportedPlatforms() const override
	{
		return SupportedPlatforms;
	}

	virtual bool SupportsFeature(EMediaFeature Feature) const override
	{
		return Feature == EMediaFeature::AudioSamples ||
				Feature == EMediaFeature::MetadataTracks ||
				Feature == EMediaFeature::VideoSamples;

	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// supported platforms
		SupportedPlatforms.Add(TEXT("Windows"));
		SupportedPlatforms.Add(TEXT("Linux"));

		// supported schemes
		SupportedUriSchemes.Add(TEXT("rmax"));

		// register player factory
		IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->RegisterPlayerFactory(*this);
		}
	}

	virtual void ShutdownModule() override
	{
		// unregister player factory
		IMediaModule* MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterPlayerFactory(*this);
		}
	}

private:
	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FRivermaxMediaFactoryModule, RivermaxMediaFactory);
