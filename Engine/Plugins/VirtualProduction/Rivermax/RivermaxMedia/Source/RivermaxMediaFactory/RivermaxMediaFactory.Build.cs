// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class RivermaxMediaFactory : ModuleRules
	{
		public RivermaxMediaFactory(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Media",
					"MediaAssets",
					"Projects",
					"RivermaxMedia"
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
				});
		}
	}
}
